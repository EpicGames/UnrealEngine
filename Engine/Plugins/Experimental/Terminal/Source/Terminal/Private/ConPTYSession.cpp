// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConPTYSession.h"

#if PLATFORM_WINDOWS

#include "Async/Async.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <windows.h>
#include "Windows/HideWindowsPlatformTypes.h"

namespace
{

struct FConPTYAPI
{
	typedef HRESULT(WINAPI* CreatePseudoConsoleFunc)(COORD Size, HANDLE hInput, HANDLE hOutput, DWORD dwFlags, void** phPC);
	typedef HRESULT(WINAPI* ResizePseudoConsoleFunc)(void* hPC, COORD Size);
	typedef void(WINAPI* ClosePseudoConsoleFunc)(void* hPC);

	CreatePseudoConsoleFunc CreatePseudoConsole = nullptr;
	ResizePseudoConsoleFunc ResizePseudoConsole = nullptr;
	ClosePseudoConsoleFunc ClosePseudoConsole = nullptr;

	/** Non-null if we loaded a bundled conpty.dll. */
	HMODULE BundledConPTYModule = nullptr;

	~FConPTYAPI()
	{
		if (BundledConPTYModule)
		{
			FreeLibrary(BundledConPTYModule);
			BundledConPTYModule = nullptr;
		}
	}
};

FConPTYAPI InitConPTYAPI()
{
	FConPTYAPI API;

	// Try bundled conpty.dll first (ships with OpenConsole.exe, bypasses system conhost.exe).
	HMODULE SourceModule = nullptr;
	TSharedPtr<IPlugin> TerminalPlugin = IPluginManager::Get().FindPlugin(TEXT("Terminal"));
	if (TerminalPlugin.IsValid())
	{
		const FString BundledConPTYPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(
			TerminalPlugin->GetBaseDir(),
			TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory(),
			TEXT("conpty.dll")));

		if (FPaths::FileExists(BundledConPTYPath))
		{
			SourceModule = LoadLibraryExW(*BundledConPTYPath, nullptr, LOAD_LIBRARY_SEARCH_DLL_LOAD_DIR | LOAD_LIBRARY_SEARCH_SYSTEM32);
			if (SourceModule)
			{
				API.BundledConPTYModule = SourceModule;
				UE_LOGF(LogTerminal, Display, "Loaded bundled ConPTY: %ls", *BundledConPTYPath);
			}
			else
			{
				UE_LOGF(LogTerminal, Warning, "Failed to load bundled conpty.dll from %ls (error %d). Falling back to system ConPTY.", *BundledConPTYPath, GetLastError());
			}
		}
	}

	if (!SourceModule)
	{
		SourceModule = GetModuleHandleW(L"kernel32.dll");
		UE_LOGF(LogTerminal, Display, "Using system ConPTY from kernel32.dll.");
	}

	if (SourceModule)
	{
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS // Unsafe conversion from FARPROC to specific function pointer type.
		API.CreatePseudoConsole = reinterpret_cast<FConPTYAPI::CreatePseudoConsoleFunc>(
			GetProcAddress(SourceModule, "CreatePseudoConsole"));
		API.ResizePseudoConsole = reinterpret_cast<FConPTYAPI::ResizePseudoConsoleFunc>(
			GetProcAddress(SourceModule, "ResizePseudoConsole"));
		API.ClosePseudoConsole = reinterpret_cast<FConPTYAPI::ClosePseudoConsoleFunc>(
			GetProcAddress(SourceModule, "ClosePseudoConsole"));
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	}
	return API;
}

// C++11 magic statics guarantee thread-safe one-time initialization.
FConPTYAPI& GetConPTYAPI()
{
	static FConPTYAPI API = InitConPTYAPI();
	return API;
}

} // anonymous namespace

FConPTYSession::FConPTYSession()
{
}

FConPTYSession::~FConPTYSession()
{
	Shutdown();
}

bool FConPTYSession::IsConPTYAvailable()
{
	const FConPTYAPI& API = GetConPTYAPI();
	return API.CreatePseudoConsole && API.ResizePseudoConsole && API.ClosePseudoConsole;
}

bool FConPTYSession::Create(const FString& ShellPath, const FString& WorkingDirectory, int32 Columns, int32 Rows)
{
	const FConPTYAPI& API = GetConPTYAPI();
	if (!API.CreatePseudoConsole)
	{
		UE_LOGF(LogTerminal, Error, "ConPTY API is not available on this system.");
		return false;
	}

	// Create the pipes.
	HANDLE InputPipeReadHandle = INVALID_HANDLE_VALUE;
	HANDLE InputPipeWriteHandle = INVALID_HANDLE_VALUE;
	HANDLE OutputPipeReadHandle = INVALID_HANDLE_VALUE;
	HANDLE OutputPipeWriteHandle = INVALID_HANDLE_VALUE;

	if (!CreatePipe(&InputPipeReadHandle, &InputPipeWriteHandle, nullptr, 0))
	{
		UE_LOGF(LogTerminal, Error, "Failed to create input pipe.");
		return false;
	}
	if (!CreatePipe(&OutputPipeReadHandle, &OutputPipeWriteHandle, nullptr, 0))
	{
		UE_LOGF(LogTerminal, Error, "Failed to create output pipe.");
		CloseHandle(InputPipeReadHandle);
		CloseHandle(InputPipeWriteHandle);
		return false;
	}

	// Create the pseudo-console.
	const int32 ClampedColumns = FMath::Clamp(Columns, 1, static_cast<int32>(SHRT_MAX));
	const int32 ClampedRows = FMath::Clamp(Rows, 1, static_cast<int32>(SHRT_MAX));
	COORD ConsoleSize = {};
	ConsoleSize.X = static_cast<SHORT>(ClampedColumns);
	ConsoleSize.Y = static_cast<SHORT>(ClampedRows);

	void* PseudoConsole = nullptr;
	HRESULT Result = API.CreatePseudoConsole(ConsoleSize, InputPipeReadHandle, OutputPipeWriteHandle, 0, &PseudoConsole);
	if (FAILED(Result))
	{
		UE_LOGF(LogTerminal, Error, "CreatePseudoConsole failed with HRESULT 0x%08X.", Result);
		CloseHandle(InputPipeReadHandle);
		CloseHandle(InputPipeWriteHandle);
		CloseHandle(OutputPipeReadHandle);
		CloseHandle(OutputPipeWriteHandle);
		return false;
	}

	// Spawn the shell process.
	FString EffectiveShellPath = ShellPath;
	if (EffectiveShellPath.IsEmpty())
	{
		// Use COMSPEC environment variable as default.
		TCHAR ComSpec[MAX_PATH];
		if (GetEnvironmentVariableW(TEXT("COMSPEC"), ComSpec, MAX_PATH) > 0)
		{
			EffectiveShellPath = ComSpec;
		}
		else
		{
			EffectiveShellPath = TEXT("cmd.exe");
		}
	}

	UE_LOGF(LogTerminal, VeryVerbose, "Resolved shell path: %ls (requested: %ls). Console size: %dx%d.", *EffectiveShellPath, ShellPath.IsEmpty() ? TEXT("<default>") : *ShellPath, ClampedColumns, ClampedRows);

	// Set up the startup info with the pseudo-console attribute.
	// First call with nullptr intentionally fails to retrieve the required buffer size.
	SIZE_T AttributeListSize = 0;
	(void)InitializeProcThreadAttributeList(nullptr, 1, 0, &AttributeListSize);
	TArray<uint8> AttributeListStorage;
	AttributeListStorage.SetNumZeroed(AttributeListSize);
	LPPROC_THREAD_ATTRIBUTE_LIST AttributeList = reinterpret_cast<LPPROC_THREAD_ATTRIBUTE_LIST>(AttributeListStorage.GetData());
	if (!AttributeList)
	{
		UE_LOGF(LogTerminal, Error, "Failed to allocate attribute list storage.");
		if (API.ClosePseudoConsole)
		{
			API.ClosePseudoConsole(PseudoConsole);
		}
		CloseHandle(InputPipeReadHandle);
		CloseHandle(InputPipeWriteHandle);
		CloseHandle(OutputPipeReadHandle);
		CloseHandle(OutputPipeWriteHandle);
		return false;
	}

	if (!InitializeProcThreadAttributeList(AttributeList, 1, 0, &AttributeListSize))
	{
		UE_LOGF(LogTerminal, Error, "InitializeProcThreadAttributeList failed.");
		if (API.ClosePseudoConsole)
		{
			API.ClosePseudoConsole(PseudoConsole);
		}
		CloseHandle(InputPipeReadHandle);
		CloseHandle(InputPipeWriteHandle);
		CloseHandle(OutputPipeReadHandle);
		CloseHandle(OutputPipeWriteHandle);
		return false;
	}

	// PROC_THREAD_ATTRIBUTE_PSEUDOCONSOLE = 0x00020016
	static const DWORD PROC_THREAD_ATTRIBUTE_PSEUDOCONSOLE_VALUE = 0x00020016;
	if (!UpdateProcThreadAttribute(
		AttributeList, 0,
		PROC_THREAD_ATTRIBUTE_PSEUDOCONSOLE_VALUE,
		PseudoConsole, sizeof(void*),
		nullptr, nullptr))
	{
		UE_LOGF(LogTerminal, Error, "UpdateProcThreadAttribute failed.");
		DeleteProcThreadAttributeList(AttributeList);
		if (API.ClosePseudoConsole)
		{
			API.ClosePseudoConsole(PseudoConsole);
		}
		CloseHandle(InputPipeReadHandle);
		CloseHandle(InputPipeWriteHandle);
		CloseHandle(OutputPipeReadHandle);
		CloseHandle(OutputPipeWriteHandle);
		return false;
	}

	STARTUPINFOEXW StartupInfoEx = {};
	StartupInfoEx.StartupInfo.cb = sizeof(STARTUPINFOEXW);
	StartupInfoEx.lpAttributeList = AttributeList;

	PROCESS_INFORMATION ProcessInfo = {};

	// CreateProcessW needs a mutable command line buffer. Quote the path in case it contains spaces.
	const FString QuotedShellPath = FString::Printf(TEXT("\"%s\""), *EffectiveShellPath);
	TArray<TCHAR> CommandLineBuffer;
	CommandLineBuffer.Append(*QuotedShellPath, QuotedShellPath.Len() + 1);

	LPCWSTR WorkingDir = WorkingDirectory.IsEmpty() ? nullptr : *WorkingDirectory;

	// CreateProcessW copies the parent's STD handle values into the child's PEB
	// regardless of bInheritHandles. When the editor is launched from a context
	// that sets std handles to pipes or files (e.g., an IDE, build system, or
	// script), the child shell inherits those handle values and uses them instead
	// of the console handles provided by ConPTY. This causes the shell's output
	// to bypass the pseudo-console entirely, resulting in a blank terminal.
	// Clearing std handles to INVALID_HANDLE_VALUE before CreateProcessW ensures
	// the child receives proper console handles from the pseudo-console.
	HANDLE SavedStdInput = GetStdHandle(STD_INPUT_HANDLE);
	HANDLE SavedStdOutput = GetStdHandle(STD_OUTPUT_HANDLE);
	HANDLE SavedStdError = GetStdHandle(STD_ERROR_HANDLE);
	if (!SetStdHandle(STD_INPUT_HANDLE, INVALID_HANDLE_VALUE)
		|| !SetStdHandle(STD_OUTPUT_HANDLE, INVALID_HANDLE_VALUE)
		|| !SetStdHandle(STD_ERROR_HANDLE, INVALID_HANDLE_VALUE))
	{
		UE_LOGF(LogTerminal, Warning, "Failed to clear standard handles before CreateProcessW (error %u). Child may inherit parent handles.", GetLastError());
	}

	BOOL bSuccess = CreateProcessW(
		*EffectiveShellPath,
		CommandLineBuffer.GetData(),
		nullptr,
		nullptr,
		0,
		EXTENDED_STARTUPINFO_PRESENT,
		nullptr,
		WorkingDir,
		&StartupInfoEx.StartupInfo,
		&ProcessInfo);

	if (!SetStdHandle(STD_INPUT_HANDLE, SavedStdInput)
		|| !SetStdHandle(STD_OUTPUT_HANDLE, SavedStdOutput)
		|| !SetStdHandle(STD_ERROR_HANDLE, SavedStdError))
	{
		UE_LOGF(LogTerminal, Warning, "Failed to restore standard handles after CreateProcessW (error %u).", GetLastError());
	}

	DeleteProcThreadAttributeList(AttributeList);

	// Close the pipe ends that belong to the child process.
	CloseHandle(InputPipeReadHandle);
	CloseHandle(OutputPipeWriteHandle);

	if (!bSuccess)
	{
		UE_LOGF(LogTerminal, Error, "CreateProcessW failed for shell: %ls (error %d).", *EffectiveShellPath, GetLastError());
		if (API.ClosePseudoConsole)
		{
			API.ClosePseudoConsole(PseudoConsole);
		}
		CloseHandle(InputPipeWriteHandle);
		CloseHandle(OutputPipeReadHandle);
		return false;
	}

	// All setup succeeded  - commit handles to member state.
	PseudoConsoleHandle = PseudoConsole;
	InputPipeWrite = InputPipeWriteHandle;
	OutputPipeRead = OutputPipeReadHandle;
	ProcessHandle = ProcessInfo.hProcess;
	ThreadHandle = ProcessInfo.hThread;
	bAlive = MakeShared<TAtomic<bool>>(true);

	// Start the background reader thread.
	ReaderRunnable = MakeUnique<FReaderRunnable>(this);
	ReaderThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(
		ReaderRunnable.Get(), TEXT("TerminalReader"), 0, TPri_Normal));

	if (!ReaderThread)
	{
		UE_LOGF(LogTerminal, Error, "Failed to create reader thread for shell: %ls", *EffectiveShellPath);
		Shutdown();
		return false;
	}

	UE_LOGF(LogTerminal, Display, "Terminal session created with shell: %ls", *EffectiveShellPath);
	return true;
}

void FConPTYSession::Shutdown()
{
	// Signal that this session is no longer alive so async callbacks are safe.
	if (bAlive)
	{
		*bAlive = false;
	}

	// Stop the reader thread first.
	if (ReaderRunnable.IsValid())
	{
		ReaderRunnable->Stop();
	}

	// Close the pseudo-console  - this will cause ReadFile on the output pipe to return.
	if (PseudoConsoleHandle)
	{
		const FConPTYAPI& API = GetConPTYAPI();
		if (API.ClosePseudoConsole)
		{
			API.ClosePseudoConsole(PseudoConsoleHandle);
		}
		PseudoConsoleHandle = nullptr;
	}

	// Wait for the reader thread to finish.
	if (ReaderThread.IsValid())
	{
		ReaderThread->WaitForCompletion();
		ReaderThread.Reset();
	}
	ReaderRunnable.Reset();

	// Terminate the shell process if still running.
	if (ProcessHandle)
	{
		DWORD ExitCode = 0;
		if (GetExitCodeProcess(static_cast<HANDLE>(ProcessHandle), &ExitCode))
		{
			if (ExitCode == STILL_ACTIVE)
			{
				TerminateProcess(static_cast<HANDLE>(ProcessHandle), 0);
			}
		}
		CloseHandle(static_cast<HANDLE>(ProcessHandle));
		ProcessHandle = nullptr;
	}

	if (ThreadHandle)
	{
		CloseHandle(static_cast<HANDLE>(ThreadHandle));
		ThreadHandle = nullptr;
	}

	if (InputPipeWrite)
	{
		CloseHandle(static_cast<HANDLE>(InputPipeWrite));
		InputPipeWrite = nullptr;
	}

	if (OutputPipeRead)
	{
		CloseHandle(static_cast<HANDLE>(OutputPipeRead));
		OutputPipeRead = nullptr;
	}
}

void FConPTYSession::WriteInput(TArrayView<const uint8> Data)
{
	if (!InputPipeWrite || Data.Num() == 0)
	{
		return;
	}

	int32 Offset = 0;
	while (Offset < Data.Num())
	{
		DWORD BytesWritten = 0;
		if (!WriteFile(static_cast<HANDLE>(InputPipeWrite), Data.GetData() + Offset, Data.Num() - Offset, &BytesWritten, nullptr))
		{
			UE_LOGF(LogTerminal, Warning, "WriteInput: WriteFile failed (error %d). Attempted %d bytes.", GetLastError(), Data.Num() - Offset);
			break;
		}
		Offset += static_cast<int32>(BytesWritten);
	}
	UE_LOGF(LogTerminal, VeryVerbose, "WriteInput: wrote %d/%d bytes to input pipe.", Offset, Data.Num());
}

void FConPTYSession::Resize(int32 Columns, int32 Rows)
{
	if (!PseudoConsoleHandle)
	{
		return;
	}

	const FConPTYAPI& API = GetConPTYAPI();
	if (API.ResizePseudoConsole)
	{
		COORD NewSize;
		NewSize.X = static_cast<SHORT>(FMath::Clamp(Columns, 1, static_cast<int32>(SHRT_MAX)));
		NewSize.Y = static_cast<SHORT>(FMath::Clamp(Rows, 1, static_cast<int32>(SHRT_MAX)));
		API.ResizePseudoConsole(PseudoConsoleHandle, NewSize);
	}
}

TArray<uint8> FConPTYSession::ConsumeOutput()
{
	FScopeLock Lock(&StagingLock);
	TArray<uint8> Output = MoveTemp(StagingBuffer);
	StagingBuffer.Reset();
	return Output;
}

bool FConPTYSession::IsRunning() const
{
	if (!ProcessHandle)
	{
		return false;
	}

	DWORD ExitCode = 0;
	if (GetExitCodeProcess(static_cast<HANDLE>(ProcessHandle), &ExitCode))
	{
		return ExitCode == STILL_ACTIVE;
	}
	return false;
}

uint32 FConPTYSession::FReaderRunnable::Run()
{
	// Cache the pipe handle locally so we do not re-read through the Owner pointer
	// on every iteration. Shutdown() guarantees OutputPipeRead stays valid until
	// after WaitForCompletion(), but caching avoids a TOCTOU gap between the null
	// check and the ReadFile call.
	const HANDLE CachedOutputPipeRead = static_cast<HANDLE>(Owner->OutputPipeRead);
	if (!CachedOutputPipeRead || CachedOutputPipeRead == INVALID_HANDLE_VALUE)
	{
		return 0;
	}

	UE_LOGF(LogTerminal, VeryVerbose, "Reader thread started. Pipe handle: %p.", CachedOutputPipeRead);

	constexpr int32 ReadBufferSize = 65536;
	constexpr int32 MaxStagingBufferSize = 16 * 1024 * 1024; // 16 MB backpressure cap.
	TArray<uint8> ReadBuffer;
	ReadBuffer.SetNumUninitialized(ReadBufferSize);
	int64 TotalBytesRead = 0;

	while (!bStopRequested)
	{
		DWORD BytesRead = 0;
		BOOL bSuccess = ReadFile(
			CachedOutputPipeRead,
			ReadBuffer.GetData(),
			ReadBufferSize,
			&BytesRead,
			nullptr);

		if (!bSuccess || BytesRead == 0)
		{
			const DWORD LastError = bSuccess ? 0 : GetLastError();
			UE_LOGF(LogTerminal, VeryVerbose, "Reader thread: ReadFile returned success=%d bytes=%d error=%d. Total bytes read: %lld.", bSuccess, BytesRead, LastError, TotalBytesRead);
			break;
		}

		TotalBytesRead += BytesRead;
		UE_LOGF(LogTerminal, VeryVerbose, "Reader thread: ReadFile returned %d bytes (total: %lld).", BytesRead, TotalBytesRead);

		{
			FScopeLock Lock(&Owner->StagingLock);
			const int32 Remaining = MaxStagingBufferSize - Owner->StagingBuffer.Num();
			if (Remaining > 0)
			{
				const int32 BytesToAppend = FMath::Min(static_cast<int32>(BytesRead), Remaining);
				Owner->StagingBuffer.Append(ReadBuffer.GetData(), BytesToAppend);
				if (BytesToAppend < static_cast<int32>(BytesRead))
				{
					UE_LOGF(LogTerminal, Warning, "Staging buffer backpressure: %d of %d bytes dropped (cap %d).",
						BytesRead - BytesToAppend, BytesRead, MaxStagingBufferSize);
				}
			}
			else
			{
				UE_LOGF(LogTerminal, Warning, "Staging buffer full: %d bytes dropped (cap %d).", BytesRead, MaxStagingBufferSize);
			}
		}
	}

	UE_LOGF(LogTerminal, VeryVerbose, "Reader thread exiting. Total bytes read: %lld. StopRequested: %d.", TotalBytesRead, bStopRequested ? 1 : 0);

	// Check for process exit.
	if (Owner->ProcessHandle)
	{
		DWORD ExitCode = 0;
		if (GetExitCodeProcess(static_cast<HANDLE>(Owner->ProcessHandle), &ExitCode) && ExitCode != STILL_ACTIVE)
		{
			AsyncTask(ENamedThreads::GameThread, [Alive = Owner->bAlive, Delegate = Owner->OnProcessExited, ExitCode]()
			{
				if (Alive && *Alive)
				{
					Delegate.ExecuteIfBound(static_cast<int32>(ExitCode));
				}
			});
		}
	}

	return 0;
}

void FConPTYSession::FReaderRunnable::Stop()
{
	bStopRequested = true;
}

#endif // PLATFORM_WINDOWS
