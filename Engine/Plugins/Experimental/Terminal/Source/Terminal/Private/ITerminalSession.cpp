// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITerminalSession.h"

#if PLATFORM_WINDOWS
#include "ConPTYSession.h"
#elif PLATFORM_UNIX || PLATFORM_MAC
#include "PosixPTYSession.h"
#endif

TSharedPtr<ITerminalSession> ITerminalSession::CreateForCurrentPlatform(FString& OutError)
{
#if PLATFORM_WINDOWS
	if (FConPTYSession::IsConPTYAvailable())
	{
		return MakeShared<FConPTYSession>();
	}
	OutError = TEXT("ConPTY API is not available on this system.");
	return nullptr;
#elif PLATFORM_UNIX || PLATFORM_MAC
	return MakeShared<FPosixPTYSession>();
#else
	OutError = TEXT("Terminal is not supported on this platform.");
	return nullptr;
#endif
}
