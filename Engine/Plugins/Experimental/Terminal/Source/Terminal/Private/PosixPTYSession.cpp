// Copyright Epic Games, Inc. All Rights Reserved.

#include "PosixPTYSession.h"

#if PLATFORM_UNIX || PLATFORM_MAC

#include "Async/Async.h"
#include "Containers/Utf8String.h"
#include "Misc/Paths.h"

#include <unistd.h>
#include <fcntl.h>
#include <signal.h>
#include <sys/ioctl.h>
#include <sys/wait.h>
#include <termios.h>

#if PLATFORM_MAC
// Forward-declare forkpty to avoid including <util.h>, which transitively
// pulls in <pwd.h>  - absent from some CI Apple SDK configurations.
extern "C" int forkpty(int*, char*, struct termios*, struct winsize*);
#else
#include <pty.h>
#endif

FPosixPTYSession::FPosixPTYSession()
{
}

FPosixPTYSession::~FPosixPTYSession()
{
	Shutdown();
}

bool FPosixPTYSession::Create(const FString& ShellPath, const FString& WorkingDirectory, int32 Columns, int32 Rows)
{
	// Set up the initial terminal window size.
	struct winsize WindowSize = {};
	WindowSize.ws_col = static_cast<unsigned short>(FMath::Clamp(Columns, 1, 32767));
	WindowSize.ws_row = static_cast<unsigned short>(FMath::Clamp(Rows, 1, 32767));

	int MasterFD = -1;
	pid_t ChildPid = forkpty(&MasterFD, nullptr, nullptr, &WindowSize);

	if (ChildPid < 0)
	{
		UE_LOGF(LogTerminal, Error, "forkpty() failed with errno %d.", errno);
		return false;
	}

	if (ChildPid == 0)
	{
		// Child process  - exec the shell.
		FString EffectiveShellPath = ShellPath;
		if (EffectiveShellPath.IsEmpty())
		{
			const char* ShellEnv = getenv("SHELL");
			if (ShellEnv && ShellEnv[0] != '\0')
			{
				EffectiveShellPath = UTF8_TO_TCHAR(ShellEnv);
			}
			else
			{
				EffectiveShellPath = TEXT("/bin/bash");
			}
		}

		if (!WorkingDirectory.IsEmpty())
		{
			const FUtf8String DirectoryUtf8(WorkingDirectory);
			if (chdir(reinterpret_cast<const char*>(*DirectoryUtf8)) != 0)
			{
				// Cannot log from the forked child; proceed in the default directory.
				// The parent will detect issues through normal shell behavior.
			}
		}

		// Set TERM so curses/readline applications behave correctly.
		setenv("TERM", "xterm-256color", 1);

		const FUtf8String ShellUtf8(EffectiveShellPath);
		const char* ShellCStr = reinterpret_cast<const char*>(*ShellUtf8);

		// Extract just the shell name for argv[0] (convention: login shells use "-bash" etc.).
		const char* ShellName = strrchr(ShellCStr, '/');
		ShellName = ShellName ? ShellName + 1 : ShellCStr;

		// execvp requires a mutable argv. Use a login-shell style argv[0].
		char LoginShellName[256];
		snprintf(LoginShellName, sizeof(LoginShellName), "-%s", ShellName);

		char* Argv[] = { LoginShellName, nullptr };
		execvp(ShellCStr, Argv);

		// exec failed  - exit the forked child.
		_exit(127);
	}

	// Parent process.
	MasterFileDescriptor = MasterFD;
	ProcessId = ChildPid;
	bAlive = MakeShared<TAtomic<bool>>(true);

	UE_LOGF(LogTerminal, VeryVerbose, "Resolved shell path: %ls (requested: %ls). Terminal size: %dx%d. PID: %d.",
		*ShellPath, ShellPath.IsEmpty() ? TEXT("<default>") : *ShellPath, Columns, Rows, static_cast<int32>(ProcessId));

	// Start the background reader thread.
	ReaderRunnable = MakeUnique<FReaderRunnable>(this);
	ReaderThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(
		ReaderRunnable.Get(), TEXT("TerminalReader"), 0, TPri_Normal));

	if (!ReaderThread)
	{
		UE_LOGF(LogTerminal, Error, "Failed to create reader thread for shell PID %d.", static_cast<int32>(ProcessId));
		Shutdown();
		return false;
	}

	UE_LOGF(LogTerminal, Display, "Terminal session created with shell PID %d.", static_cast<int32>(ProcessId));
	return true;
}

void FPosixPTYSession::Shutdown()
{
	// Signal that this session is no longer alive so async callbacks are safe.
	if (bAlive)
	{
		*bAlive = false;
	}

	// Stop the reader thread first.
	if (ReaderRunnable.IsValid())
	{
		ReaderRunnable->Stop();
	}

	// Close the master FD  - this signals EOF to the reader thread's read() call.
	if (MasterFileDescriptor >= 0)
	{
		close(MasterFileDescriptor);
		MasterFileDescriptor = -1;
	}

	// Wait for the reader thread to finish.
	if (ReaderThread.IsValid())
	{
		ReaderThread->WaitForCompletion();
		ReaderThread.Reset();
	}
	ReaderRunnable.Reset();

	// Reap the child process.
	if (ProcessId > 0)
	{
		int Status = 0;
		pid_t Result = waitpid(ProcessId, &Status, WNOHANG);
		if (Result < 0)
		{
			UE_LOGF(LogTerminal, Warning, "Shutdown: waitpid(%d) failed (errno %d).", static_cast<int32>(ProcessId), errno);
		}
		else if (Result == 0)
		{
			// Child still running  - send SIGHUP then SIGKILL.
			if (kill(ProcessId, SIGHUP) != 0)
			{
				UE_LOGF(LogTerminal, Warning, "Shutdown: kill(%d, SIGHUP) failed (errno %d).", static_cast<int32>(ProcessId), errno);
			}
			usleep(100000); // 100ms grace period.
			Result = waitpid(ProcessId, &Status, WNOHANG);
			if (Result == 0)
			{
				if (kill(ProcessId, SIGKILL) != 0)
				{
					UE_LOGF(LogTerminal, Warning, "Shutdown: kill(%d, SIGKILL) failed (errno %d).", static_cast<int32>(ProcessId), errno);
				}
				waitpid(ProcessId, &Status, 0);
			}
			else if (Result < 0)
			{
				UE_LOGF(LogTerminal, Warning, "Shutdown: waitpid(%d) after SIGHUP failed (errno %d).", static_cast<int32>(ProcessId), errno);
			}
		}
		ProcessId = -1;
	}
}

void FPosixPTYSession::WriteInput(TArrayView<const uint8> Data)
{
	if (MasterFileDescriptor < 0 || Data.Num() == 0)
	{
		return;
	}

	int32 Offset = 0;
	while (Offset < Data.Num())
	{
		ssize_t BytesWritten = write(MasterFileDescriptor, Data.GetData() + Offset, Data.Num() - Offset);
		if (BytesWritten <= 0)
		{
			if (BytesWritten < 0 && errno == EINTR)
			{
				continue;
			}
			UE_LOGF(LogTerminal, Warning, "WriteInput: write() returned %zd (errno %d). Attempted %d bytes.", BytesWritten, BytesWritten < 0 ? errno : 0, Data.Num() - Offset);
			break;
		}
		Offset += static_cast<int32>(BytesWritten);
	}
	UE_LOGF(LogTerminal, VeryVerbose, "WriteInput: wrote %d/%d bytes to master FD.", Offset, Data.Num());
}

void FPosixPTYSession::Resize(int32 Columns, int32 Rows)
{
	if (MasterFileDescriptor < 0)
	{
		return;
	}

	struct winsize WindowSize = {};
	WindowSize.ws_col = static_cast<unsigned short>(FMath::Clamp(Columns, 1, 32767));
	WindowSize.ws_row = static_cast<unsigned short>(FMath::Clamp(Rows, 1, 32767));
	if (ioctl(MasterFileDescriptor, TIOCSWINSZ, &WindowSize) != 0)
	{
		UE_LOGF(LogTerminal, Warning, "Resize: ioctl(TIOCSWINSZ) failed (errno %d) for %dx%d.", errno, Columns, Rows);
	}
}

TArray<uint8> FPosixPTYSession::ConsumeOutput()
{
	FScopeLock Lock(&StagingLock);
	TArray<uint8> Output = MoveTemp(StagingBuffer);
	StagingBuffer.Reset();
	return Output;
}

bool FPosixPTYSession::IsRunning() const
{
	if (ProcessId <= 0)
	{
		return false;
	}

	// Use kill with signal 0 to check liveness without reaping the child.
	// waitpid would reap the process, racing with Shutdown() and the reader thread.
	return kill(ProcessId, 0) == 0;
}

uint32 FPosixPTYSession::FReaderRunnable::Run()
{
	const int CachedMasterFD = Owner->MasterFileDescriptor;
	if (CachedMasterFD < 0)
	{
		return 0;
	}

	UE_LOGF(LogTerminal, VeryVerbose, "Reader thread started. Master FD: %d.", CachedMasterFD);

	constexpr int32 ReadBufferSize = 65536;
	constexpr int32 MaxStagingBufferSize = 16 * 1024 * 1024; // 16 MB backpressure cap.
	TArray<uint8> ReadBuffer;
	ReadBuffer.SetNumUninitialized(ReadBufferSize);
	int64 TotalBytesRead = 0;

	while (!bStopRequested)
	{
		ssize_t BytesRead = read(CachedMasterFD, ReadBuffer.GetData(), ReadBufferSize);

		if (BytesRead <= 0)
		{
			if (BytesRead < 0 && errno == EINTR)
			{
				continue;
			}
			UE_LOGF(LogTerminal, VeryVerbose, "Reader thread: read() returned %zd (errno %d). Total bytes read: %lld.",
				BytesRead, BytesRead < 0 ? errno : 0, TotalBytesRead);
			break;
		}

		TotalBytesRead += BytesRead;
		UE_LOGF(LogTerminal, VeryVerbose, "Reader thread: read() returned %zd bytes (total: %lld).", BytesRead, TotalBytesRead);

		{
			FScopeLock Lock(&Owner->StagingLock);
			const int32 Remaining = MaxStagingBufferSize - Owner->StagingBuffer.Num();
			if (Remaining > 0)
			{
				const int32 BytesToAppend = FMath::Min(static_cast<int32>(BytesRead), Remaining);
				Owner->StagingBuffer.Append(ReadBuffer.GetData(), BytesToAppend);
				if (BytesToAppend < static_cast<int32>(BytesRead))
				{
					UE_LOGF(LogTerminal, Warning, "Staging buffer backpressure: %zd of %zd bytes dropped (cap %d).",
						BytesRead - BytesToAppend, BytesRead, MaxStagingBufferSize);
				}
			}
			else
			{
				UE_LOGF(LogTerminal, Warning, "Staging buffer full: %zd bytes dropped (cap %d).", BytesRead, MaxStagingBufferSize);
			}
		}
	}

	UE_LOGF(LogTerminal, VeryVerbose, "Reader thread exiting. Total bytes read: %lld. StopRequested: %d.", TotalBytesRead, bStopRequested ? 1 : 0);

	// Check for process exit.
	if (Owner->ProcessId > 0)
	{
		int Status = 0;
		pid_t Result = waitpid(Owner->ProcessId, &Status, WNOHANG);
		if (Result > 0)
		{
			int32 ExitCode = WIFEXITED(Status) ? WEXITSTATUS(Status) : -1;
			AsyncTask(ENamedThreads::GameThread, [Alive = Owner->bAlive, Delegate = Owner->OnProcessExited, ExitCode]()
			{
				if (Alive && *Alive)
				{
					Delegate.ExecuteIfBound(ExitCode);
				}
			});
		}
	}

	return 0;
}

void FPosixPTYSession::FReaderRunnable::Stop()
{
	bStopRequested = true;
}

#endif // PLATFORM_UNIX || PLATFORM_MAC
