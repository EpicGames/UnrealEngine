// Copyright Epic Games, Inc. All Rights Reserved.

#include "STerminal.h"

#include "ITerminalSession.h"
#include "TerminalKeyTranslator.h"
#include "TerminalSettings.h"
#include "TerminalSubsystem.h"
#include "TerminalUtilities.h"
#include "Editor.h"

#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Rendering/DrawElements.h"
#include "Widgets/Layout/SScrollBar.h"

static TAutoConsoleVariable<bool> CVarTerminalDebugGlyphs(
	TEXT("Terminal.DebugGlyphs"),
	false,
	TEXT("When true, draws per-cell glyph boundary boxes to diagnose font measurement vs rendering mismatches."),
	ECVF_Default);

namespace UE::Terminal
{
	static TArray<TWeakPtr<STerminal>>& GetInstanceRegistry()
	{
		static TArray<TWeakPtr<STerminal>> Instances;
		return Instances;
	}
}

TArray<TSharedRef<STerminal>> STerminal::GetAllInstances()
{
	check(IsInGameThread());
	TArray<TWeakPtr<STerminal>>& Registry = UE::Terminal::GetInstanceRegistry();
	TArray<TSharedRef<STerminal>> Result;
	Result.Reserve(Registry.Num());
	for (int32 Index = Registry.Num() - 1; Index >= 0; --Index)
	{
		if (TSharedPtr<STerminal> Pinned = Registry[Index].Pin())
		{
			Result.Add(Pinned.ToSharedRef());
		}
		else
		{
			Registry.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
	return Result;
}

void STerminal::Construct(const FArguments& InArgs)
{
	// Set up the font. Buffer and session creation are deferred to the first paint
	// so we know the actual widget dimensions and avoid a spurious resize.
	const UTerminalSettings* Settings = GetDefault<UTerminalSettings>();
	const FString FontFamily = Settings ? Settings->FontFamily : TEXT("Consolas");
	const int32 FontSize = Settings ? Settings->FontSize : 10;

	// Resolve font: try configured font, then common monospace fallbacks, then engine font.
	FString FontFilePath = TerminalUtilities::ResolveSystemFontPath(FontFamily);
	if (FontFilePath.IsEmpty())
	{
		for (const TCHAR* Fallback : TerminalUtilities::GetFallbackFontNames())
		{
			if (FontFamily != Fallback)
			{
				FontFilePath = TerminalUtilities::ResolveSystemFontPath(Fallback);
				if (!FontFilePath.IsEmpty())
				{
					break;
				}
			}
		}
	}
	
	if (FontFilePath.IsEmpty())
	{
		UE_LOGF(LogTerminal, Warning, "No system monospace font found. Falling back to engine default.");
		FontFilePath = TerminalUtilities::GetEngineFallbackFontPath();
	}
	
	UE_LOGF(LogTerminal, Display, "Using terminal font: %ls", *FontFilePath);

	TerminalFont = MakeShared<FStandaloneCompositeFont>(NAME_None, FontFilePath, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);

	// Add symbol/emoji fallback sub-typefaces so glyphs not in the monospace
	// font (box drawing, arrows, math operators, emoji, etc.) still render.
	{
		for (const TCHAR* SymbolFontName : TerminalUtilities::GetSymbolFontNames())
		{
			const FString SymbolFontPath = TerminalUtilities::ResolveSystemFontPath(SymbolFontName);
			if (!SymbolFontPath.IsEmpty())
			{
				FCompositeSubFont& SymbolSubFont = TerminalFont->SubTypefaces.AddDefaulted_GetRef();
				// Broad range covering symbols, box drawing, block elements, geometric shapes,
				// arrows, mathematical operators, Braille, dingbats, and miscellaneous symbols.
				SymbolSubFont.CharacterRanges.Add(FInt32Range(FInt32Range::BoundsType::Inclusive(0x2000), FInt32Range::BoundsType::Inclusive(0x2BFF)));
				// Supplemental arrows and math
				SymbolSubFont.CharacterRanges.Add(FInt32Range(FInt32Range::BoundsType::Inclusive(0x2E80), FInt32Range::BoundsType::Inclusive(0x2FFF)));
				SymbolSubFont.Typeface.AppendFont(TEXT("Regular"), SymbolFontPath, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
				break;
			}
		}

		for (const TCHAR* EmojiFontName : TerminalUtilities::GetEmojiFontNames())
		{
			const FString EmojiFontPath = TerminalUtilities::ResolveSystemFontPath(EmojiFontName);
			if (!EmojiFontPath.IsEmpty())
			{
				FCompositeSubFont& EmojiSubFont = TerminalFont->SubTypefaces.AddDefaulted_GetRef();
				// Supplemental symbols and pictographs (0x2600-0x27BF already covered by Symbol sub-typeface).
				EmojiSubFont.CharacterRanges.Add(FInt32Range(FInt32Range::BoundsType::Inclusive(0x1F300), FInt32Range::BoundsType::Inclusive(0x1FAFF)));
				EmojiSubFont.Typeface.AppendFont(TEXT("Regular"), EmojiFontPath, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
				break;
			}
		}
	}

	FontInfo = FSlateFontInfo(TerminalFont, static_cast<float>(FontSize));
	FontInfo.FontFallback = EFontFallback::FF_Max;

	MeasureCellSize();

	ScrollBar = InArgs._ExternalScrollbar;
	if (ScrollBar.IsValid())
	{
		ScrollBar->SetOnUserScrolled(FOnUserScrolled::CreateSP(this, &STerminal::HandleScrollBarScrolled));
		ScrollBar->SetState(0.0f, 1.0f);
	}

	UE_LOGF(LogTerminal, VeryVerbose, "STerminal::Construct complete. CellSize=%.1fx%.1f", CellWidth, CellHeight);

	// Register tick for consuming ConPTY output.
	RegisterActiveTimer(0.016f, FWidgetActiveTimerDelegate::CreateSP(this, &STerminal::OnTick));

	// Register cursor blink timer (~530ms).
	RegisterActiveTimer(0.530f, FWidgetActiveTimerDelegate::CreateSP(this, &STerminal::OnCursorBlink));

	TArray<TWeakPtr<STerminal>>& Registry = UE::Terminal::GetInstanceRegistry();
	Registry.RemoveAll([](const TWeakPtr<STerminal>& Weak) { return !Weak.IsValid(); });
	Registry.Add(SharedThis(this));
}

void STerminal::ExecuteCommand(const FString& Command)
{
	if (!Session.IsValid())
	{
		return;
	}

	const FUtf8String CommandUtf8(Command + TEXT("\r"));
	Session->WriteInput(MakeArrayView(reinterpret_cast<const uint8*>(*CommandUtf8), CommandUtf8.Len()));
}

bool STerminal::IsSessionRunning() const
{
	return Session.IsValid() && Session->IsRunning();
}

STerminal::~STerminal()
{
	UE_LOGF(LogTerminal, VeryVerbose, "~STerminal: total bytes consumed by parser: %lld. Cursor at (%d,%d). Initialized: %d.", TotalBytesConsumed, Buffer.Cursor.Column, Buffer.Cursor.Row, bInitialized ? 1 : 0);

	if (Session.IsValid())
	{
		Session->Shutdown();
		Session.Reset();
	}
}

FVector2D STerminal::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(ViewportColumns * CellWidth, ViewportRows * CellHeight);
}

int32 STerminal::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STerminal::OnPaint);

	// Detect size changes for resize handling. Slate's OnPaint is const, but deferred
	// initialization and viewport resize must mutate state on the first (or resized) paint.
	// This is a standard Slate pattern for geometry-dependent lazy init.
	// When not yet initialized, always attempt UpdateViewportDimensions so that
	// initialization is not skipped when the first paint arrives with zero geometry
	// (which would match the zero-initialized LastAllocatedSize and be a no-op).
	const FVector2D CurrentSize = AllottedGeometry.GetLocalSize();
	if (!bInitialized || !CurrentSize.Equals(LastAllocatedSize, 1.0f))
	{
		LastAllocatedSize = CurrentSize;
		const_cast<STerminal*>(this)->UpdateViewportDimensions(AllottedGeometry);
	}

	// Draw full background.
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor(Buffer.DefaultCell.Background));

	if (bInitialized)
	{
		PaintBackground(AllottedGeometry, OutDrawElements, ++LayerId);
		PaintText(AllottedGeometry, OutDrawElements, ++LayerId);
		PaintForeground(AllottedGeometry, OutDrawElements, ++LayerId);
	}

	return LayerId;
}

FColor STerminal::ComputeForegroundColor(const FTerminalCell& Cell)
{
	FColor Color = Cell.Foreground;
	if (Cell.Attributes & ETerminalAttribute::Inverse)
	{
		Color = Cell.Background;
	}

	if (Cell.Attributes & ETerminalAttribute::Bold)
	{
		Color.R = static_cast<uint8>(FMath::Min(255, Color.R + 60));
		Color.G = static_cast<uint8>(FMath::Min(255, Color.G + 60));
		Color.B = static_cast<uint8>(FMath::Min(255, Color.B + 60));
	}
	else if (Cell.Attributes & ETerminalAttribute::Dim)
	{
		Color.R /= 2;
		Color.G /= 2;
		Color.B /= 2;
	}

	return Color;
}

void STerminal::PaintBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STerminal::PaintBackground);

	const FColor DefaultBackground = Buffer.DefaultCell.Background;
	const int32 TotalRows = Buffer.GetTotalRows();
	const int32 Columns = Buffer.GetColumns();

	for (int32 ViewportRow = 0; ViewportRow < ViewportRows; ++ViewportRow)
	{
		const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, ScrollOffset);
		if (AbsoluteRow < 0 || AbsoluteRow >= TotalRows)
		{
			continue;
		}

		const float Y = ViewportRow * CellHeight;
		int32 Col = 0;
		while (Col < Columns)
		{
			const FTerminalCell& StartCell = Buffer.GetCell(AbsoluteRow, Col);

			FColor BackgroundColor = StartCell.Background;
			if (StartCell.Attributes & ETerminalAttribute::Inverse)
			{
				BackgroundColor = StartCell.Foreground;
			}

			// Batch consecutive cells with same background.
			int32 RunEnd = Col + 1;
			while (RunEnd < Columns)
			{
				const FTerminalCell& NextCell = Buffer.GetCell(AbsoluteRow, RunEnd);

				FColor NextBackground = NextCell.Background;
				if (NextCell.Attributes & ETerminalAttribute::Inverse)
				{
					NextBackground = NextCell.Foreground;
				}

				if (NextBackground != BackgroundColor)
				{
					break;
				}
				RunEnd++;
			}

			if (BackgroundColor != DefaultBackground)
			{
				const float X = Col * CellWidth;
				const float RunWidth = (RunEnd - Col) * CellWidth;
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(RunWidth, CellHeight), FSlateLayoutTransform(FVector2D(X, Y))),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					FLinearColor(BackgroundColor));
			}

			Col = RunEnd;
		}
	}
}

void STerminal::PaintText(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STerminal::PaintText);

	const int32 TotalRows = Buffer.GetTotalRows();
	const int32 Columns = Buffer.GetColumns();

	for (int32 ViewportRow = 0; ViewportRow < ViewportRows; ++ViewportRow)
	{
		const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, ScrollOffset);
		if (AbsoluteRow < 0 || AbsoluteRow >= TotalRows)
		{
			continue;
		}

		const float Y = ViewportRow * CellHeight;

		// Batch runs of monospace-safe characters with the same foreground/attributes.
		// Non-ASCII characters are rendered individually at their cell position to
		// maintain grid alignment.
		int32 Col = 0;
		while (Col < Columns)
		{
			const FTerminalCell& StartCell = Buffer.GetCell(AbsoluteRow, Col);
			const FColor ForegroundColor = ComputeForegroundColor(StartCell);
			const bool bStartMonoSafe = TerminalUtilities::IsMonospaceSafe(StartCell.Character);

			if (bStartMonoSafe)
			{
				// Batch consecutive monospace-safe characters with same foreground and attributes.
				FString BatchText;
				BatchText.Reserve(Columns - Col);
				BatchText.AppendChar(StartCell.Character);
				int32 RunEnd = Col + 1;
				while (RunEnd < Columns)
				{
					const FTerminalCell& NextCell = Buffer.GetCell(AbsoluteRow, RunEnd);
					if (!TerminalUtilities::IsMonospaceSafe(NextCell.Character))
					{
						break;
					}

					const FColor NextForeground = ComputeForegroundColor(NextCell);
					if (NextForeground != ForegroundColor || NextCell.Attributes != StartCell.Attributes)
					{
						break;
					}
					BatchText.AppendChar(NextCell.Character);
					RunEnd++;
				}

				const float X = Col * CellWidth;
				const float RunWidth = (RunEnd - Col) * CellWidth;
				const FLinearColor DrawColor(ForegroundColor);
				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(RunWidth, CellHeight), FSlateLayoutTransform(FVector2D(X, Y))),
					BatchText,
					FontInfo,
					ESlateDrawEffect::None,
					DrawColor);

				// Underline decoration for the entire run.
				if (StartCell.Attributes & ETerminalAttribute::Underline)
				{
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(RunWidth, 1.0f), FSlateLayoutTransform(FVector2D(X, Y + CellHeight - 1.0f))),
						FAppStyle::GetBrush("WhiteBrush"),
						ESlateDrawEffect::None,
						DrawColor);
				}

				// Strikethrough decoration for the entire run.
				if (StartCell.Attributes & ETerminalAttribute::Strikethrough)
				{
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(RunWidth, 1.0f), FSlateLayoutTransform(FVector2D(X, Y + CellHeight * 0.5f))),
						FAppStyle::GetBrush("WhiteBrush"),
						ESlateDrawEffect::None,
						DrawColor);
				}

				Col = RunEnd;
			}
			else
			{
				// Non-monospace-safe character: render at cell position without
				// per-cell clipping. Fallback font glyphs may have advances
				// slightly different from the monospace cell width; clipping to
				// exact cell boundaries causes cut edges and anti-aliasing
				// shadow artifacts. Any minor glyph overflow is covered by
				// adjacent cells' backgrounds drawn on the layer below.
				const float CharX = Col * CellWidth;
				const FString CharString(1, &StartCell.Character);
				const FLinearColor DrawColor(ForegroundColor);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(
						FVector2D(CellWidth, CellHeight), FSlateLayoutTransform(FVector2D(CharX, Y))),
					CharString,
					FontInfo,
					ESlateDrawEffect::None,
					DrawColor);

				// Underline.
				if (StartCell.Attributes & ETerminalAttribute::Underline)
				{
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(CellWidth, 1.0f), FSlateLayoutTransform(FVector2D(CharX, Y + CellHeight - 1.0f))),
						FAppStyle::GetBrush("WhiteBrush"),
						ESlateDrawEffect::None,
						DrawColor);
				}

				// Strikethrough.
				if (StartCell.Attributes & ETerminalAttribute::Strikethrough)
				{
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(CellWidth, 1.0f), FSlateLayoutTransform(FVector2D(CharX, Y + CellHeight * 0.5f))),
						FAppStyle::GetBrush("WhiteBrush"),
						ESlateDrawEffect::None,
						DrawColor);
				}

				Col++;
			}
		}
	}
}

void STerminal::PaintForeground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STerminal::PaintForeground);

	// Selection highlight (translucent overlay on top of text).
	if (bHasSelection || bSelecting)
	{
		const FLinearColor SelectionColor(SelectionHighlightColor.ReinterpretAsLinear() * FLinearColor(1.0f, 1.0f, 1.0f, 0.35f));
		const int32 TotalRows = Buffer.GetTotalRows();
		const int32 Columns = Buffer.GetColumns();

		for (int32 ViewportRow = 0; ViewportRow < ViewportRows; ++ViewportRow)
		{
			const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, ScrollOffset);
			if (AbsoluteRow < 0 || AbsoluteRow >= TotalRows)
			{
				continue;
			}

			const float Y = ViewportRow * CellHeight;
			int32 Col = 0;
			while (Col < Columns)
			{
				if (!IsCellSelected(AbsoluteRow, Col))
				{
					Col++;
					continue;
				}

				// Batch consecutive selected cells.
				int32 RunEnd = Col + 1;
				while (RunEnd < Columns && IsCellSelected(AbsoluteRow, RunEnd))
				{
					RunEnd++;
				}

				const float X = Col * CellWidth;
				const float RunWidth = (RunEnd - Col) * CellWidth;
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(RunWidth, CellHeight), FSlateLayoutTransform(FVector2D(X, Y))),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					SelectionColor);

				Col = RunEnd;
			}
		}
	}

	// Cursor.
	if (Parser.bCursorVisible && Buffer.Cursor.bBlinkOn && ScrollOffset == 0)
	{
		const float CursorX = Buffer.Cursor.Column * CellWidth;
		const float CursorY = Buffer.Cursor.Row * CellHeight;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(CellWidth, CellHeight), FSlateLayoutTransform(FVector2D(CursorX, CursorY))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FLinearColor(CursorDisplayColor));
	}

	// Per-cell glyph boundaries: draw a uniquely colored box for each character.
	if (CVarTerminalDebugGlyphs.GetValueOnGameThread())
	{
		const int32 TotalRows = Buffer.GetTotalRows();
		const int32 Columns = Buffer.GetColumns();

		for (int32 ViewportRow = 0; ViewportRow < ViewportRows; ++ViewportRow)
		{
			const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, ScrollOffset);
			if (AbsoluteRow < 0 || AbsoluteRow >= TotalRows)
			{
				continue;
			}

			const float Y = ViewportRow * CellHeight;

			for (int32 Col = 0; Col < Columns; ++Col)
			{
				const FTerminalCell& Cell = Buffer.GetCell(AbsoluteRow, Col);
				if (Cell.Character == TEXT(' '))
				{
					continue;
				}

				const FLinearColor GlyphColor = FLinearColor::MakeFromHSV8(static_cast<uint8>((Col * 47 + ViewportRow * 131) % 256), 200, 255).CopyWithNewOpacity(0.25f);
				const FLinearColor GlyphBorderColor = GlyphColor.CopyWithNewOpacity(0.7f);

				const FVector2D GlyphPosition(Col * CellWidth, Y);
				const FVector2D GlyphSize(CellWidth, CellHeight);

				// Fill.
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(GlyphSize, FSlateLayoutTransform(GlyphPosition)),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					GlyphColor);

				// Left edge.
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2D(1.0f, CellHeight), FSlateLayoutTransform(GlyphPosition)),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					GlyphBorderColor);
			}
		}
	}
}

FReply STerminal::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	if (!Session.IsValid())
	{
		return FReply::Unhandled();
	}

	const TCHAR Character = InCharacterEvent.GetCharacter();

	// Skip control characters  - they are handled in OnKeyDown (Enter, Tab, arrows, etc.).
	if (Character < 0x20)
	{
		return FReply::Unhandled();
	}

	// Encode as UTF-8 and write to the session.
	const FUtf8String CharUtf8(FString(1, &Character));
	UE_LOGF(LogTerminal, VeryVerbose, "OnKeyChar: U+%04X (%d bytes UTF-8).", static_cast<uint32>(Character), CharUtf8.Len());
	Session->WriteInput(MakeArrayView(reinterpret_cast<const uint8*>(*CharUtf8), CharUtf8.Len()));

	// Reset scroll to bottom on input.
	ScrollOffset = 0;
	UpdateScrollBar();

	return FReply::Handled();
}

TOptional<FReply> STerminal::HandleShiftArrowSelection(const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();
	const bool bArrow = Key == EKeys::Up || Key == EKeys::Down || Key == EKeys::Left || Key == EKeys::Right;
	if (!bArrow || !InKeyEvent.IsShiftDown() || InKeyEvent.IsControlDown())
	{
		return {};
	}

	if (!bHasSelection)
	{
		// Start selection at cursor (cursor row is in viewport-local space with ScrollOffset=0).
		const int32 CursorAbsoluteRow = Buffer.GetAbsoluteRow(Buffer.Cursor.Row, 0);
		SelectionAnchor = FIntPoint(Buffer.Cursor.Column, CursorAbsoluteRow);
		SelectionActive = SelectionAnchor;
		bHasSelection = true;
	}

	if (Key == EKeys::Up)
	{
		if (SelectionActive.Y > 0)
		{
			SelectionActive.Y -= 1;
			const int32 TopVisibleRow = Buffer.GetAbsoluteRow(0, ScrollOffset);
			if (SelectionActive.Y < TopVisibleRow)
			{
				ScrollOffset = FMath::Min(ScrollOffset + 1, Buffer.GetScrollbackLength());
				UpdateScrollBar();
			}
		}
	}
	else if (Key == EKeys::Down)
	{
		if (SelectionActive.Y < Buffer.GetTotalRows() - 1)
		{
			SelectionActive.Y += 1;
			const int32 BottomVisibleRow = Buffer.GetAbsoluteRow(ViewportRows - 1, ScrollOffset);
			if (SelectionActive.Y > BottomVisibleRow)
			{
				ScrollOffset = FMath::Max(ScrollOffset - 1, 0);
				UpdateScrollBar();
			}
		}
	}
	else if (Key == EKeys::Left)
	{
		if (SelectionActive.X > 0)
		{
			SelectionActive.X -= 1;
		}
		else if (SelectionActive.Y > 0)
		{
			SelectionActive.Y -= 1;
			SelectionActive.X = Buffer.GetColumns() - 1;
			const int32 TopVisibleRow = Buffer.GetAbsoluteRow(0, ScrollOffset);
			if (SelectionActive.Y < TopVisibleRow)
			{
				ScrollOffset = FMath::Min(ScrollOffset + 1, Buffer.GetScrollbackLength());
				UpdateScrollBar();
			}
		}
	}
	else // EKeys::Right
	{
		if (SelectionActive.X < Buffer.GetColumns() - 1)
		{
			SelectionActive.X += 1;
		}
		else if (SelectionActive.Y < Buffer.GetTotalRows() - 1)
		{
			SelectionActive.Y += 1;
			SelectionActive.X = 0;
			const int32 BottomVisibleRow = Buffer.GetAbsoluteRow(ViewportRows - 1, ScrollOffset);
			if (SelectionActive.Y > BottomVisibleRow)
			{
				ScrollOffset = FMath::Max(ScrollOffset - 1, 0);
				UpdateScrollBar();
			}
		}
	}

	Invalidate(EInvalidateWidgetReason::Paint);
	return FReply::Handled();
}

TOptional<FReply> STerminal::HandleCopyShortcut(const FKeyEvent& InKeyEvent, bool bCopyPasteModifier)
{
	// Only consume the event when there is a selection to copy. Without a selection on Ctrl+C we fall
	// through so the translator sends ETX (interrupt) to the shell.
	if (!bCopyPasteModifier || InKeyEvent.GetKey() != EKeys::C || !bHasSelection)
	{
		return {};
	}

	FString SelectedText = GetSelectedText();
	SelectedText.ReplaceInline(TEXT("\r\n"), TEXT("\n"));
	SelectedText.ReplaceInline(TEXT("\r"), TEXT("\n"));
	SelectedText.ReplaceInline(TEXT("\n"), LINE_TERMINATOR);
	FPlatformApplicationMisc::ClipboardCopy(*SelectedText);
	ClearSelection();
	return FReply::Handled();
}

TOptional<FReply> STerminal::HandlePasteShortcut(const FKeyEvent& InKeyEvent, bool bCopyPasteModifier)
{
	if (!bCopyPasteModifier || InKeyEvent.GetKey() != EKeys::V)
	{
		return {};
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (!ClipboardText.IsEmpty())
	{
		ClipboardText.ReplaceInline(TEXT("\r\n"), TEXT("\n"));
		ClipboardText.ReplaceInline(TEXT("\r"), TEXT("\n"));
		const FUtf8String ClipboardUtf8(ClipboardText);
		TArray<uint8> UTF8Data;
		if (Parser.bBracketedPaste)
		{
			static const uint8 BracketOpen[] = { 0x1B, '[', '2', '0', '0', '~' };
			UTF8Data.Append(BracketOpen, UE_ARRAY_COUNT(BracketOpen));
		}
		UTF8Data.Append(reinterpret_cast<const uint8*>(*ClipboardUtf8), ClipboardUtf8.Len());
		if (Parser.bBracketedPaste)
		{
			static const uint8 BracketClose[] = { 0x1B, '[', '2', '0', '1', '~' };
			UTF8Data.Append(BracketClose, UE_ARRAY_COUNT(BracketClose));
		}
		Session->WriteInput(UTF8Data);
		ScrollOffset = 0;
		UpdateScrollBar();
	}
	return FReply::Handled();
}

FReply STerminal::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Shift+Arrow selection runs before the session check: it does not need a live shell.
	if (TOptional<FReply> SelectionReply = HandleShiftArrowSelection(InKeyEvent))
	{
		return *SelectionReply;
	}

	if (!Session.IsValid())
	{
		return FReply::Unhandled();
	}

	// Copy/paste use Cmd on macOS (iTerm2/Terminal.app convention) and Ctrl everywhere else.
	// The PTY-directed translator below is unaffected: Cmd is never forwarded to the shell on macOS.
#if PLATFORM_MAC
	const bool bCopyPasteModifier = InKeyEvent.IsCommandDown();
#else
	const bool bCopyPasteModifier = InKeyEvent.IsControlDown();
#endif

	if (TOptional<FReply> CopyReply = HandleCopyShortcut(InKeyEvent, bCopyPasteModifier))
	{
		return *CopyReply;
	}
	if (TOptional<FReply> PasteReply = HandlePasteShortcut(InKeyEvent, bCopyPasteModifier))
	{
		return *PasteReply;
	}

	UE::Terminal::FKeyTranslationOptions TranslationOptions;
	TranslationOptions.bApplicationCursorKeys = Parser.bApplicationCursorKeys;
	const TArray<uint8> SequenceData = UE::Terminal::TranslateKeyToBytes(InKeyEvent, TranslationOptions);

	if (SequenceData.Num() > 0)
	{
		UE_LOGF(LogTerminal, VeryVerbose, "OnKeyDown: %ls (%d bytes).", *InKeyEvent.GetKey().ToString(), SequenceData.Num());
		Session->WriteInput(SequenceData);
		ScrollOffset = 0;
		UpdateScrollBar();
	}

	// Consume the event regardless. Printable characters arrive via OnKeyChar; modifier-only
	// presses are harmlessly consumed. Letting them bubble would let the editor hijack single-letter
	// shortcuts (e.g. 'f' for viewport focus) while the terminal has focus. When no session is active,
	// the early return above already lets events propagate.
	return FReply::Handled();
}

FReply STerminal::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Forward mouse press to the application when tracking is active.
	if (Parser.MouseTrackingMode != FVTParser::EMouseTrackingMode::None)
	{
		if (Session.IsValid())
		{
			const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);
			const int32 Row = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.Y) / CellHeight), 0, ViewportRows - 1);
			int32 Button = 0;
			if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton) { Button = 1; }
			else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton) { Button = 2; }
			SendMouseEvent(Button, Col, Row, false);
		}
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const FIntPoint Cell = PixelToCell(MyGeometry, MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
		const double CurrentTime = FPlatformTime::Seconds();

		// Detect multi-click (double/triple).
		if (CurrentTime - LastClickTime < 0.4 && Cell == LastClickCell)
		{
			ClickCount++;
		}
		else
		{
			ClickCount = 1;
		}
		LastClickTime = CurrentTime;
		LastClickCell = Cell;

		if (ClickCount == 2)
		{
			// Double-click: select word.
			const int32 AbsRow = Cell.Y;
			int32 WordStart = Cell.X;
			int32 WordEnd = Cell.X;

			// Expand backward.
			while (WordStart > 0)
			{
				const FTerminalCell& PrevCell = Buffer.GetCell(AbsRow, WordStart - 1);
				if (FChar::IsWhitespace(PrevCell.Character) || PrevCell.Character == TEXT('\0'))
				{
					break;
				}
				WordStart--;
			}
			// Expand forward.
			while (WordEnd < Buffer.GetColumns() - 1)
			{
				const FTerminalCell& NextCell = Buffer.GetCell(AbsRow, WordEnd + 1);
				if (FChar::IsWhitespace(NextCell.Character) || NextCell.Character == TEXT('\0'))
				{
					break;
				}
				WordEnd++;
			}

			SelectionAnchor = FIntPoint(WordStart, AbsRow);
			SelectionActive = FIntPoint(WordEnd, AbsRow);
			bHasSelection = true;
			bSelecting = false;
			Invalidate(EInvalidateWidgetReason::Paint);
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
		else if (ClickCount >= 3)
		{
			// Triple-click: select entire row.
			const int32 AbsRow = Cell.Y;
			SelectionAnchor = FIntPoint(0, AbsRow);
			SelectionActive = FIntPoint(Buffer.GetColumns() - 1, AbsRow);
			bHasSelection = true;
			bSelecting = false;
			Invalidate(EInvalidateWidgetReason::Paint);
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		// Single click: start selection.
		SelectionAnchor = Cell;
		SelectionActive = Cell;
		bSelecting = true;
		bHasSelection = false;

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply STerminal::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Forward mouse release to the application when tracking is active.
	if (Parser.MouseTrackingMode != FVTParser::EMouseTrackingMode::None)
	{
		if (Session.IsValid())
		{
			const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);
			const int32 Row = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.Y) / CellHeight), 0, ViewportRows - 1);
			int32 Button = 0;
			if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton) { Button = 1; }
			else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton) { Button = 2; }
			SendMouseEvent(Button, Col, Row, true);
		}
		return FReply::Handled().ReleaseMouseCapture();
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bSelecting)
	{
		bSelecting = false;
		SelectionAutoScrollDirection = 0;
		if (SelectionAnchor != SelectionActive)
		{
			bHasSelection = true;
		}
		else
		{
			bHasSelection = false;
		}
		Invalidate(EInvalidateWidgetReason::Paint);
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Handled().ReleaseMouseCapture();
}

FReply STerminal::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Forward mouse motion to the application when any-event tracking (1003) is active,
	// or button-event tracking (1002) with a button held.
	if (Parser.MouseTrackingMode != FVTParser::EMouseTrackingMode::None)
	{
		// Determine which button is held: 0=left, 1=middle, 2=right, 3=none.
		int32 HeldButton = 3;
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton)) { HeldButton = 0; }
		else if (MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton)) { HeldButton = 1; }
		else if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton)) { HeldButton = 2; }

		if (Parser.MouseTrackingMode == FVTParser::EMouseTrackingMode::Any
			|| (Parser.MouseTrackingMode == FVTParser::EMouseTrackingMode::ButtonEvent && HeldButton != 3))
		{
			if (Session.IsValid())
			{
				const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);
				const int32 Row = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.Y) / CellHeight), 0, ViewportRows - 1);
				const int32 MotionButton = 32 + HeldButton; // 32 = motion flag; button encoded in low bits
				SendMouseEvent(MotionButton, Col, Row, false);
			}
			return FReply::Handled();
		}
	}

	if (bSelecting)
	{
		const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		const float ViewportHeight = ViewportRows * CellHeight;
		const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);

		if (LocalPosition.Y < 0.0f)
		{
			// Mouse above viewport: scroll up to show older content.
			const int32 LinesToScroll = FMath::Clamp(FMath::CeilToInt32(-LocalPosition.Y / CellHeight), 1, 5);
			const int32 MaxScrollback = Buffer.GetScrollbackLength();
			ScrollOffset = FMath::Clamp(ScrollOffset + LinesToScroll, 0, MaxScrollback);
			SelectionActive = FIntPoint(Col, Buffer.GetAbsoluteRow(0, ScrollOffset));
			SelectionAutoScrollDirection = LinesToScroll;
			SelectionAutoScrollAccumulator = 0.0f;
			UpdateScrollBar();
		}
		else if (LocalPosition.Y >= ViewportHeight)
		{
			// Mouse below viewport: scroll down to show newer content.
			const int32 LinesToScroll = FMath::Clamp(FMath::CeilToInt32((LocalPosition.Y - ViewportHeight) / CellHeight), 1, 5);
			ScrollOffset = FMath::Max(0, ScrollOffset - LinesToScroll);
			SelectionActive = FIntPoint(Col, Buffer.GetAbsoluteRow(ViewportRows - 1, ScrollOffset));
			SelectionAutoScrollDirection = -LinesToScroll;
			SelectionAutoScrollAccumulator = 0.0f;
			UpdateScrollBar();
		}
		else
		{
			// Mouse inside viewport: normal selection update.
			SelectionAutoScrollDirection = 0;
			SelectionActive = PixelToCell(MyGeometry, LocalPosition);
		}

		Invalidate(EInvalidateWidgetReason::Paint);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply STerminal::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsControlDown())
	{
		UTerminalSettings* Settings = GetMutableDefault<UTerminalSettings>();
		const int32 Delta = MouseEvent.GetWheelDelta() > 0 ? 1 : -1;
		Settings->FontSize = FMath::Clamp(Settings->FontSize + Delta, 6, 72);
		Settings->SaveConfig();
		FontInfo = FSlateFontInfo(TerminalFont, static_cast<float>(Settings->FontSize));
		FontInfo.FontFallback = EFontFallback::FF_Max;
		LastAllocatedSize = FVector2D::ZeroVector; // Force cell remeasure on next paint.
		Invalidate(EInvalidateWidgetReason::Layout);
		return FReply::Handled();
	}

	// When mouse tracking is active, forward wheel events to the application instead of scrolling.
	if (Parser.MouseTrackingMode != FVTParser::EMouseTrackingMode::None)
	{
		if (Session.IsValid())
		{
			const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);
			const int32 Row = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.Y) / CellHeight), 0, ViewportRows - 1);
			const int32 WheelButton = MouseEvent.GetWheelDelta() > 0 ? 64 : 65; // 64 = wheel up, 65 = wheel down
			SendMouseEvent(WheelButton, Col, Row, false);
		}
		return FReply::Handled();
	}

	// Alternate screen has no scrollback - ignore scroll attempts.
	if (Buffer.IsAlternateBufferActive())
	{
		return FReply::Handled();
	}

	const float Delta = MouseEvent.GetWheelDelta();
	const int32 LinesToScroll = FMath::Clamp(FMath::RoundToInt32(Delta) * 3, -ViewportRows, ViewportRows);

	const int32 MaxScrollback = Buffer.GetScrollbackLength();
	ScrollOffset = FMath::Clamp(ScrollOffset + LinesToScroll, 0, MaxScrollback);

	UpdateScrollBar();
	Invalidate(EInvalidateWidgetReason::Paint);
	return FReply::Handled();
}

FReply STerminal::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Already handled via click counting in OnMouseButtonDown.
	return FReply::Handled();
}

FReply STerminal::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	bHasFocus = true;
	Buffer.Cursor.bBlinkOn = true;
	Invalidate(EInvalidateWidgetReason::Paint);
	return FReply::Handled();
}

void STerminal::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	bHasFocus = false;
	Buffer.Cursor.bBlinkOn = true;

	// Clear active drag-selection to avoid ambiguous state.
	bSelecting = false;
	SelectionAutoScrollDirection = 0;

	Invalidate(EInvalidateWidgetReason::Paint);
}

FCursorReply STerminal::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FCursorReply::Cursor(EMouseCursor::TextEditBeam);
}

EActiveTimerReturnType STerminal::OnTick(double InCurrentTime, float InDeltaTime)
{
	if (!bInitialized || !Session.IsValid())
	{
		return EActiveTimerReturnType::Continue;
	}

	// Continuous auto-scroll while drag-selecting outside the viewport.
	if (bSelecting && SelectionAutoScrollDirection != 0)
	{
		SelectionAutoScrollAccumulator += InDeltaTime;
		static constexpr float AutoScrollInterval = 0.05f; // ~20 lines/second
		if (SelectionAutoScrollAccumulator >= AutoScrollInterval)
		{
			SelectionAutoScrollAccumulator -= AutoScrollInterval;
			const int32 MaxScrollback = Buffer.GetScrollbackLength();
			ScrollOffset = FMath::Clamp(ScrollOffset + SelectionAutoScrollDirection, 0, MaxScrollback);

			if (SelectionAutoScrollDirection > 0)
			{
				// Scrolling up: extend selection to top of viewport.
				SelectionActive.Y = Buffer.GetAbsoluteRow(0, ScrollOffset);
			}
			else
			{
				// Scrolling down: extend selection to bottom of viewport.
				SelectionActive.Y = Buffer.GetAbsoluteRow(ViewportRows - 1, ScrollOffset);
			}

			UpdateScrollBar();
			Invalidate(EInvalidateWidgetReason::Paint);
		}
	}

	// Consume output from the terminal session.
	TArray<uint8> Output = Session->ConsumeOutput();

	// Prepend any pending data from previous ticks.
	if (PendingParseData.Num() > 0)
	{
		PendingParseData.Append(Output);
		Output = MoveTemp(PendingParseData);
		PendingParseData.Reset();
	}

	if (Output.Num() > 0)
	{
		LastOutputTime = InCurrentTime;

		if (!bReceivedFirstOutput)
		{
			bReceivedFirstOutput = true;
			UE_LOGF(LogTerminal, VeryVerbose, "OnTick: first output consumed (%d bytes). Cursor at (%d,%d).", Output.Num(), Buffer.Cursor.Column, Buffer.Cursor.Row);
		}

		const int32 BytesToParse = FMath::Min(Output.Num(), ParseBudgetPerTick);
		Parser.Parse(Output.GetData(), BytesToParse);
		TotalBytesConsumed += BytesToParse;

		UE_LOGF(LogTerminal, VeryVerbose, "OnTick: parsed %d bytes (total consumed: %lld). Cursor at (%d,%d). Parser state: %ls.", BytesToParse, TotalBytesConsumed, Buffer.Cursor.Column, Buffer.Cursor.Row, *Parser.GetDebugStateName());

		// Hex dump early output to diagnose startup issues.
		if (TotalBytesConsumed <= 4096)
		{
			FString HexDump;
			HexDump.Reserve(BytesToParse * 3 + 1);
			for (int32 i = 0; i < BytesToParse; ++i)
			{
				HexDump += FString::Printf(TEXT("%02X "), Output[i]);
			}
			UE_LOGF(LogTerminal, VeryVerbose, "OnTick: raw hex [%d bytes]: %ls", BytesToParse, *HexDump);
		}

		// Defer remainder if we exceeded the budget.
		if (BytesToParse < Output.Num())
		{
			PendingParseData.Append(Output.GetData() + BytesToParse, Output.Num() - BytesToParse);
		}

		// Drain any responses the parser generated (DA, DSR replies).
		if (Parser.ResponseBuffer.Num() > 0)
		{
			Session->WriteInput(Parser.ResponseBuffer);
			Parser.ResponseBuffer.Reset();
		}

		// Defer repaint while synchronized output (DEC mode 2026) is active - the app
		// is mid-frame and will send ?2026l when done, at which point we paint once.
		if (!Parser.bSynchronizedOutput)
		{
			UpdateScrollBar();
			Invalidate(EInvalidateWidgetReason::Paint);
		}

		OnOutputReceived.Broadcast(BytesToParse);
	}
	else if (LastOutputTime > 0.0 && InCurrentTime - LastHeartbeatLogTime >= 5.0)
	{
		LastHeartbeatLogTime = InCurrentTime;
		const double SilenceDuration = InCurrentTime - LastOutputTime;
		UE_LOGF(LogTerminal, VeryVerbose, "OnTick: no output for %.1fs. Process running: %d. Total bytes consumed: %lld. HasFocus: %d.", SilenceDuration, Session->IsRunning() ? 1 : 0, TotalBytesConsumed, bHasFocus ? 1 : 0);
	}

	return EActiveTimerReturnType::Continue;
}

EActiveTimerReturnType STerminal::OnCursorBlink(double InCurrentTime, float InDeltaTime)
{
	if (!bInitialized || !bHasFocus)
	{
		return EActiveTimerReturnType::Continue;
	}

	Buffer.Cursor.bBlinkOn = !Buffer.Cursor.bBlinkOn;
	Invalidate(EInvalidateWidgetReason::Paint);
	return EActiveTimerReturnType::Continue;
}

void STerminal::MeasureCellSize(float LayoutScale)
{
	// Get the glyph advance directly from the font cache at the actual rendering scale.
	// The element batcher renders text at the geometry's accumulated scale (DPI), so we must
	// query metrics at that same scale and divide back to local space. Otherwise font hinting
	// causes the rendered per-character advance to drift from the grid CellWidth.
	const TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();
	FCharacterList& CharacterList = FontCache->GetCharacterList(FontInfo, LayoutScale);
	const FCharacterEntry& Entry = CharacterList.GetCharacter(TEXT('M'), FontInfo.FontFallback);

	CellWidth = FMath::Max(1.0f, static_cast<float>(Entry.XAdvance) / LayoutScale);
	CellHeight = FMath::Max(1.0f, static_cast<float>(CharacterList.GetMaxHeight()) / LayoutScale);
}

void STerminal::InitializeTerminal()
{
	const UTerminalSettings* Settings = GetDefault<UTerminalSettings>();
	const int32 ScrollbackLimit = Settings ? Settings->ScrollbackLimit : 131072;

	UE_LOGF(LogTerminal, VeryVerbose, "InitializeTerminal: %dx%d, scrollback=%d.", ViewportColumns, ViewportRows, ScrollbackLimit);

	Buffer.Initialize(ViewportColumns, ViewportRows, ScrollbackLimit);
	Parser.SetBuffer(&Buffer);

	// Apply color scheme.
	if (GEditor)
	{
		if (UTerminalSubsystem* Subsystem = GEditor->GetEditorSubsystem<UTerminalSubsystem>())
		{
			ApplyColorScheme(Subsystem->GetActiveColorScheme());
		}
	}

	// Initialize tab stops.
	Parser.InitializeTabStops();

	// Create the platform-specific PTY session at the correct dimensions.
	FString SessionError;
	Session = ITerminalSession::CreateForCurrentPlatform(SessionError);

	if (!Session.IsValid() && !SessionError.IsEmpty())
	{
		UE_LOGF(LogTerminal, Error, "%ls", *SessionError);

		const FUtf8String DisplayErrorUtf8(FString::Printf(TEXT("\r\nError: %s\r\n"), *SessionError));
		Parser.Parse(reinterpret_cast<const uint8*>(*DisplayErrorUtf8), DisplayErrorUtf8.Len());
	}

	if (Session.IsValid())
	{
		const FString ShellPath = Settings ? Settings->ShellExecutablePath : FString();
		const FString WorkingDirectory = FPaths::ConvertRelativePathToFull(FPaths::RootDir());
		Session->OnProcessExited.BindSP(this, &STerminal::HandleProcessExited);
		if (!Session->Create(ShellPath, WorkingDirectory, ViewportColumns, ViewportRows))
		{
			UE_LOGF(LogTerminal, Error, "Failed to create terminal session.");
			Session.Reset();

			const FUtf8String ErrorUtf8(TEXT("\r\nError: Failed to create terminal session. Check the output log for details.\r\n"));
			Parser.Parse(reinterpret_cast<const uint8*>(*ErrorUtf8), ErrorUtf8.Len());
		}
		else if (Settings)
		{
			for (const FString& StartupCommand : Settings->StartupCommands)
			{
				if (!StartupCommand.IsEmpty())
				{
					ExecuteCommand(StartupCommand);
				}
			}
		}
	}

	bInitialized = true;
	UpdateScrollBar();
}

void STerminal::UpdateViewportDimensions(const FGeometry& AllottedGeometry)
{
	// Re-measure cell size at the actual DPI scale so the grid matches rendered glyph advances.
	MeasureCellSize(AllottedGeometry.Scale);

	const FVector2D Size = AllottedGeometry.GetLocalSize();
	const int32 NewColumns = FMath::Max(1, FMath::FloorToInt32(static_cast<float>(Size.X) / CellWidth));
	const int32 NewRows = FMath::Max(1, FMath::FloorToInt32(static_cast<float>(Size.Y) / CellHeight));

	if (!bInitialized)
	{
		// Require minimum dimensions to avoid initializing ConPTY at unusable sizes
		// (e.g. 1x1 during layout restoration or docking animation). The shell would
		// produce garbled output in a micro-terminal. Initialization is deferred until
		// a subsequent paint provides reasonable geometry.
		static constexpr int32 MinimumColumns = 2;
		static constexpr int32 MinimumRows = 1;
		if (NewColumns < MinimumColumns || NewRows < MinimumRows)
		{
			UE_LOGF(LogTerminal, VeryVerbose, "Deferring initialization: geometry %dx%d below minimum %dx%d.", NewColumns, NewRows, MinimumColumns, MinimumRows);
			return;
		}

		UE_LOGF(LogTerminal, VeryVerbose, "First valid geometry received: %dx%d. Initializing terminal.", NewColumns, NewRows);
		ViewportColumns = NewColumns;
		ViewportRows = NewRows;
		InitializeTerminal();
		return;
	}

	if (NewColumns != ViewportColumns || NewRows != ViewportRows)
	{
		UE_LOGF(LogTerminal, VeryVerbose, "Viewport resized: %dx%d -> %dx%d.", ViewportColumns, ViewportRows, NewColumns, NewRows);

		ViewportColumns = NewColumns;
		ViewportRows = NewRows;
		Buffer.Resize(ViewportColumns, ViewportRows);

		if (Session.IsValid())
		{
			Session->Resize(ViewportColumns, ViewportRows);
		}

		// Re-initialize tab stops for new column count.
		Parser.InitializeTabStops();
		UpdateScrollBar();
	}
}

FIntPoint STerminal::PixelToCell(const FGeometry& MyGeometry, const FVector2D& LocalPosition) const
{
	if (Buffer.GetColumns() <= 0 || ViewportRows <= 0)
	{
		return FIntPoint(0, 0);
	}

	const int32 Col = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.X) / CellWidth), 0, Buffer.GetColumns() - 1);
	const int32 ViewportRow = FMath::Clamp(FMath::FloorToInt32(static_cast<float>(LocalPosition.Y) / CellHeight), 0, ViewportRows - 1);
	const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, ScrollOffset);
	return FIntPoint(Col, AbsoluteRow);
}

bool STerminal::IsCellSelected(int32 AbsoluteRow, int32 Column) const
{
	if (!bHasSelection && !bSelecting)
	{
		return false;
	}

	// Normalize selection range so Start <= End.
	FIntPoint Start = SelectionAnchor;
	FIntPoint End = SelectionActive;
	if (Start.Y > End.Y || (Start.Y == End.Y && Start.X > End.X))
	{
		Swap(Start, End);
	}

	if (AbsoluteRow < Start.Y || AbsoluteRow > End.Y)
	{
		return false;
	}

	if (AbsoluteRow == Start.Y && AbsoluteRow == End.Y)
	{
		return Column >= Start.X && Column <= End.X;
	}

	if (AbsoluteRow == Start.Y)
	{
		return Column >= Start.X;
	}

	if (AbsoluteRow == End.Y)
	{
		return Column <= End.X;
	}

	return true; // Rows between start and end are fully selected.
}

FString STerminal::GetSelectedText() const
{
	if (!bHasSelection)
	{
		return FString();
	}

	FIntPoint Start = SelectionAnchor;
	FIntPoint End = SelectionActive;
	if (Start.Y > End.Y || (Start.Y == End.Y && Start.X > End.X))
	{
		Swap(Start, End);
	}

	return Buffer.GetTextInRange(Start.Y, Start.X, End.Y, End.X);
}

void STerminal::ClearSelection()
{
	bSelecting = false;
	bHasSelection = false;
	SelectionAutoScrollDirection = 0;
	SelectionAnchor = FIntPoint(-1, -1);
	SelectionActive = FIntPoint(-1, -1);
	Invalidate(EInvalidateWidgetReason::Paint);
}

void STerminal::ApplyColorScheme(const FTerminalColorScheme& Scheme)
{
	Buffer.DefaultCell.Foreground = Scheme.DefaultForeground.ToFColor(true);
	Buffer.DefaultCell.Background = Scheme.DefaultBackground.ToFColor(true);
	Parser.CurrentForeground = Buffer.DefaultCell.Foreground;
	Parser.CurrentBackground = Buffer.DefaultCell.Background;

	SelectionHighlightColor = Scheme.SelectionColor.ToFColor(true);
	CursorDisplayColor = Scheme.CursorColor.ToFColor(true);
	CursorDisplayColor.A = 180; // Translucent overlay for cursor visibility

	// Copy the 16-color ANSI palette from the scheme.
	const int32 PaletteCount = FMath::Min(Scheme.Palette.Num(), 16);
	for (int32 Index = 0; Index < PaletteCount; ++Index)
	{
		Parser.AnsiPalette[Index] = Scheme.Palette[Index].ToFColor(true);
	}
}

void STerminal::UpdateScrollBar()
{
	if (!ScrollBar.IsValid())
	{
		return;
	}

	// Hide scrollbar in alternate screen buffer - no scrollback exists.
	if (Buffer.IsAlternateBufferActive())
	{
		ScrollBar->SetVisibility(EVisibility::Collapsed);
		return;
	}
	ScrollBar->SetVisibility(EVisibility::Visible);

	const int32 ScrollbackLength = Buffer.GetScrollbackLength();
	if (ScrollbackLength <= 0)
	{
		ScrollBar->SetState(0.0f, 1.0f);
		return;
	}

	const float TotalLines = static_cast<float>(ScrollbackLength + ViewportRows);
	const float ThumbSize = static_cast<float>(ViewportRows) / TotalLines;
	// ScrollOffset=0 means bottom (most recent), ScrollOffset=max means top (oldest).
	// SScrollBar offset is in [0, 1-ThumbSize]: 0=top, 1-ThumbSize=bottom.
	const float ScrollFraction = static_cast<float>(ScrollOffset) / static_cast<float>(ScrollbackLength);
	const float ThumbOffset = (1.0f - ScrollFraction) * (1.0f - ThumbSize);
	ScrollBar->SetState(ThumbOffset, ThumbSize);
}

void STerminal::HandleScrollBarScrolled(float InScrollOffsetFraction)
{
	if (Buffer.IsAlternateBufferActive())
	{
		return;
	}

	const int32 ScrollbackLength = Buffer.GetScrollbackLength();
	if (ScrollbackLength <= 0)
	{
		return;
	}

	const float TotalLines = static_cast<float>(ScrollbackLength + ViewportRows);
	const float ThumbSize = static_cast<float>(ViewportRows) / TotalLines;
	// Invert: scrollbar offset 0 = top = max ScrollOffset, offset (1-ThumbSize) = bottom = ScrollOffset 0.
	const float MaxOffset = FMath::Max(1.0f - ThumbSize, UE_SMALL_NUMBER);
	const float ScrollFraction = 1.0f - InScrollOffsetFraction / MaxOffset;
	ScrollOffset = FMath::Clamp(FMath::RoundToInt32(ScrollFraction * static_cast<float>(ScrollbackLength)), 0, ScrollbackLength);
	UpdateScrollBar();
	Invalidate(EInvalidateWidgetReason::Paint);
}

void STerminal::SendMouseEvent(int32 Button, int32 Column, int32 Row, bool bRelease)
{
	if (!Session.IsValid())
	{
		return;
	}

	// Terminal mouse coordinates are 1-based.
	const int32 Col1 = Column + 1;
	const int32 Row1 = Row + 1;

	if (Parser.bSGRMouseEncoding)
	{
		// SGR format: CSI < Button ; Col ; Row M (press) or m (release).
		const TCHAR Suffix = bRelease ? TEXT('m') : TEXT('M');
		const FString Sequence = FString::Printf(TEXT("\x1B[<%d;%d;%d%c"), Button, Col1, Row1, Suffix);
		const FTCHARToUTF8 Utf8(*Sequence);
		Session->WriteInput(MakeArrayView(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length()));
	}
	else
	{
		// Legacy X10 format: ESC [ M Cb Cx Cy (all 1-based, offset by 32).
		const uint8 Cb = static_cast<uint8>((bRelease ? 3 : Button) + 32);
		const uint8 Cx = static_cast<uint8>(FMath::Min(Col1 + 32, 255));
		const uint8 Cy = static_cast<uint8>(FMath::Min(Row1 + 32, 255));
		const uint8 Bytes[] = { 0x1B, '[', 'M', Cb, Cx, Cy };
		Session->WriteInput(Bytes);
	}
}

void STerminal::HandleProcessExited(int32 ExitCode)
{
	UE_LOGF(LogTerminal, Display, "Shell process exited with code %d.", ExitCode);

	// Drain any remaining output the reader thread accumulated before the process exited.
	// Without this, the shell's final output (error messages, etc.) would be lost when
	// Session.Reset() destroys the staging buffer.
	if (Session.IsValid())
	{
		const TArray<uint8> RemainingOutput = Session->ConsumeOutput();
		if (RemainingOutput.Num() > 0)
		{
			Parser.Parse(RemainingOutput.GetData(), RemainingOutput.Num());
		}
	}

	// Write a visual marker into the terminal so the user can see the exit.
	const FUtf8String ExitUtf8(FString::Printf(TEXT("\r\n[Process exited with code %d]\r\n"), ExitCode));
	Parser.Parse(reinterpret_cast<const uint8*>(*ExitUtf8), ExitUtf8.Len());

	bSessionDead = true;
	Session.Reset();
	UpdateScrollBar();
	Invalidate(EInvalidateWidgetReason::Paint);
}
