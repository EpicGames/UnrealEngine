// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalBuffer.h"

FTerminalBuffer::FTerminalBuffer()
{
}

void FTerminalBuffer::Initialize(int32 InColumns, int32 InViewportRows, int32 InScrollbackLimit)
{
	Columns = FMath::Max(1, InColumns);
	ViewportRows = FMath::Max(1, InViewportRows);
	ScrollbackLimit = FMath::Max(0, InScrollbackLimit);

	const int32 MaxRows = ViewportRows + ScrollbackLimit;
	Rows.SetNum(MaxRows);
	RingStart = 0;
	TotalRowsWritten = ViewportRows;

	for (int32 Index = 0; Index < ViewportRows; ++Index)
	{
		EnsureRow(Index);
	}

	Cursor = FTerminalCursor();
}

void FTerminalBuffer::Resize(int32 InColumns, int32 InViewportRows)
{
	if (InColumns == Columns && InViewportRows == ViewportRows)
	{
		return;
	}

	// Collect the current content into a linear array.
	const int32 OldTotalRows = GetTotalRows();
	TArray<TArray<FTerminalCell>> OldContent;
	OldContent.Reserve(OldTotalRows);
	for (int32 Index = 0; Index < OldTotalRows; ++Index)
	{
		const int32 RingIndex = GetRingIndex(Index);
		if (Rows.IsValidIndex(RingIndex) && Rows[RingIndex].Num() > 0)
		{
			OldContent.Add(MoveTemp(Rows[RingIndex]));
		}
		else
		{
			TArray<FTerminalCell> BlankRow;
			BlankRow.SetNum(Columns);
			for (FTerminalCell& Cell : BlankRow)
			{
				Cell = DefaultCell;
			}
			OldContent.Add(MoveTemp(BlankRow));
		}
	}

	// Reinitialize with new dimensions.
	Columns = FMath::Max(1, InColumns);
	ViewportRows = FMath::Max(1, InViewportRows);

	const int32 MaxRows = ViewportRows + ScrollbackLimit;
	Rows.SetNum(MaxRows);
	RingStart = 0;

	// Copy back as many rows as fit, adjusting column width.
	const int32 RowsToCopy = FMath::Min(OldContent.Num(), MaxRows);
	const int32 StartOffset = FMath::Max(0, OldContent.Num() - RowsToCopy);
	TotalRowsWritten = FMath::Max(RowsToCopy, ViewportRows);

	for (int32 Index = 0; Index < MaxRows; ++Index)
	{
		const int32 SourceIndex = StartOffset + Index;
		if (Index < RowsToCopy && SourceIndex < OldContent.Num())
		{
			TArray<FTerminalCell>& SourceRow = OldContent[SourceIndex];
			Rows[Index].SetNum(Columns);
			for (int32 Col = 0; Col < Columns; ++Col)
			{
				Rows[Index][Col] = (Col < SourceRow.Num()) ? SourceRow[Col] : DefaultCell;
			}
		}
		else
		{
			Rows[Index].SetNum(Columns);
			for (FTerminalCell& Cell : Rows[Index])
			{
				Cell = DefaultCell;
			}
		}
	}

	// Clamp cursor.
	Cursor.Row = FMath::Clamp(Cursor.Row, 0, ViewportRows - 1);
	Cursor.Column = FMath::Clamp(Cursor.Column, 0, Columns - 1);
}

const FTerminalCell& FTerminalBuffer::GetCell(int32 AbsoluteRow, int32 Column) const
{
	static const FTerminalCell EmptyCell;

	if (AbsoluteRow < 0 || AbsoluteRow >= GetTotalRows() || Column < 0 || Column >= Columns)
	{
		return EmptyCell;
	}

	const int32 RingIndex = GetRingIndex(AbsoluteRow);
	if (Rows.IsValidIndex(RingIndex) && Rows[RingIndex].Num() > Column)
	{
		return Rows[RingIndex][Column];
	}
	return EmptyCell;
}

void FTerminalBuffer::SetCell(int32 AbsoluteRow, int32 Column, const FTerminalCell& Cell)
{
	if (AbsoluteRow < 0 || AbsoluteRow >= GetTotalRows() || Column < 0 || Column >= Columns)
	{
		return;
	}

	const int32 RingIndex = GetRingIndex(AbsoluteRow);
	EnsureRow(RingIndex);
	Rows[RingIndex][Column] = Cell;
}

void FTerminalBuffer::PushNewLine()
{
	const int32 MaxRows = ViewportRows + ScrollbackLimit;

	if (TotalRowsWritten < MaxRows)
	{
		const int32 RingIndex = GetRingIndex(TotalRowsWritten);
		EnsureRow(RingIndex);
		// Clear the new row.
		for (FTerminalCell& Cell : Rows[RingIndex])
		{
			Cell = DefaultCell;
		}
		TotalRowsWritten++;
	}
	else
	{
		// Ring buffer is full  - reuse the oldest row.
		const int32 RingIndex = RingStart;
		EnsureRow(RingIndex);
		for (FTerminalCell& Cell : Rows[RingIndex])
		{
			Cell = DefaultCell;
		}
		RingStart = (RingStart + 1) % MaxRows;
	}
}

int32 FTerminalBuffer::GetAbsoluteRow(int32 ViewportRow, int32 ScrollOffset) const
{
	const int32 ScrollbackLength = GetScrollbackLength();
	const int32 BaseOffset = FMath::Clamp(ScrollOffset, 0, ScrollbackLength);
	return (GetTotalRows() - ViewportRows - BaseOffset) + ViewportRow;
}

void FTerminalBuffer::Clear()
{
	const int32 MaxRows = ViewportRows + ScrollbackLimit;
	RingStart = 0;
	TotalRowsWritten = ViewportRows;

	for (int32 Index = 0; Index < ViewportRows; ++Index)
	{
		EnsureRow(Index);
		for (FTerminalCell& Cell : Rows[Index])
		{
			Cell = DefaultCell;
		}
	}

	Cursor = FTerminalCursor();
}

int32 FTerminalBuffer::GetScrollbackLength() const
{
	return FMath::Max(0, GetTotalRows() - ViewportRows);
}

int32 FTerminalBuffer::GetTotalRows() const
{
	return TotalRowsWritten;
}

void FTerminalBuffer::ScrollRegionUp(int32 TopRow, int32 BottomRow, int32 Count)
{
	if (Count <= 0 || TopRow >= BottomRow)
	{
		return;
	}

	const int32 ScrollbackLength = GetScrollbackLength();

	for (int32 Row = TopRow; Row <= BottomRow; ++Row)
	{
		const int32 SourceRow = Row + Count;
		const int32 AbsoluteDestination = GetAbsoluteRow(Row, 0);

		if (SourceRow <= BottomRow)
		{
			const int32 AbsoluteSource = GetAbsoluteRow(SourceRow, 0);
			for (int32 Col = 0; Col < Columns; ++Col)
			{
				SetCell(AbsoluteDestination, Col, GetCell(AbsoluteSource, Col));
			}
		}
		else
		{
			for (int32 Col = 0; Col < Columns; ++Col)
			{
				SetCell(AbsoluteDestination, Col, DefaultCell);
			}
		}
	}
}

void FTerminalBuffer::ScrollRegionDown(int32 TopRow, int32 BottomRow, int32 Count)
{
	if (Count <= 0 || TopRow >= BottomRow)
	{
		return;
	}

	for (int32 Row = BottomRow; Row >= TopRow; --Row)
	{
		const int32 SourceRow = Row - Count;
		const int32 AbsoluteDestination = GetAbsoluteRow(Row, 0);

		if (SourceRow >= TopRow)
		{
			const int32 AbsoluteSource = GetAbsoluteRow(SourceRow, 0);
			for (int32 Col = 0; Col < Columns; ++Col)
			{
				SetCell(AbsoluteDestination, Col, GetCell(AbsoluteSource, Col));
			}
		}
		else
		{
			for (int32 Col = 0; Col < Columns; ++Col)
			{
				SetCell(AbsoluteDestination, Col, DefaultCell);
			}
		}
	}
}

void FTerminalBuffer::FillRegion(int32 AbsoluteRowStart, int32 ColumnStart, int32 AbsoluteRowEnd, int32 ColumnEnd, const FTerminalCell& Cell)
{
	for (int32 Row = AbsoluteRowStart; Row <= AbsoluteRowEnd; ++Row)
	{
		const int32 ColBegin = (Row == AbsoluteRowStart) ? ColumnStart : 0;
		const int32 ColFinish = (Row == AbsoluteRowEnd) ? ColumnEnd : Columns - 1;

		for (int32 Col = ColBegin; Col <= ColFinish; ++Col)
		{
			SetCell(Row, Col, Cell);
		}
	}
}

FString FTerminalBuffer::GetTextInRange(int32 StartRow, int32 StartColumn, int32 EndRow, int32 EndColumn) const
{
	FString Result;

	for (int32 Row = StartRow; Row <= EndRow; ++Row)
	{
		const int32 ColBegin = (Row == StartRow) ? StartColumn : 0;
		const int32 ColFinish = (Row == EndRow) ? EndColumn : Columns - 1;

		for (int32 Col = ColBegin; Col <= ColFinish; ++Col)
		{
			const FTerminalCell& Cell = GetCell(Row, Col);
			Result.AppendChar(Cell.Character);
		}

		if (Row < EndRow)
		{
			// Trim trailing spaces from the row before adding newline.
			while (Result.Len() > 0 && Result[Result.Len() - 1] == TEXT(' '))
			{
				Result.LeftChopInline(1);
			}
			Result.AppendChar(TEXT('\n'));
		}
	}

	return Result;
}

void FTerminalBuffer::ClearScrollback()
{
	if (GetScrollbackLength() == 0)
	{
		return;
	}

	// Collect current viewport content.
	TArray<TArray<FTerminalCell>> ViewportContent;
	ViewportContent.SetNum(ViewportRows);
	for (int32 Row = 0; Row < ViewportRows; ++Row)
	{
		const int32 AbsRow = GetAbsoluteRow(Row, 0);
		const int32 RingIndex = GetRingIndex(AbsRow);
		if (Rows.IsValidIndex(RingIndex) && Rows[RingIndex].Num() > 0)
		{
			ViewportContent[Row] = Rows[RingIndex];
		}
		else
		{
			ViewportContent[Row].SetNum(Columns);
			for (FTerminalCell& Cell : ViewportContent[Row])
			{
				Cell = DefaultCell;
			}
		}
	}

	// Reset ring buffer with only viewport rows.
	RingStart = 0;
	TotalRowsWritten = ViewportRows;
	for (int32 Index = 0; Index < ViewportRows; ++Index)
	{
		Rows[Index] = MoveTemp(ViewportContent[Index]);
	}
	for (int32 Index = ViewportRows; Index < Rows.Num(); ++Index)
	{
		Rows[Index].Reset();
	}
}

void FTerminalBuffer::ActivateAlternateBuffer()
{
	if (bAlternateBufferActive)
	{
		return;
	}

	bAlternateBufferActive = true;
	SavedMainRows = MoveTemp(Rows);
	SavedRingStart = RingStart;
	SavedTotalRowsWritten = TotalRowsWritten;
	SavedCursor = Cursor;

	// Initialize a clean alternate buffer (viewport-sized only, no scrollback).
	RingStart = 0;
	TotalRowsWritten = ViewportRows;
	Rows.SetNum(ViewportRows);
	for (int32 Index = 0; Index < ViewportRows; ++Index)
	{
		EnsureRow(Index);
		for (FTerminalCell& Cell : Rows[Index])
		{
			Cell = DefaultCell;
		}
	}
	Cursor = FTerminalCursor();
}

void FTerminalBuffer::DeactivateAlternateBuffer()
{
	if (!bAlternateBufferActive)
	{
		return;
	}

	bAlternateBufferActive = false;
	Rows = MoveTemp(SavedMainRows);
	RingStart = SavedRingStart;
	TotalRowsWritten = SavedTotalRowsWritten;
	Cursor = SavedCursor;
}

int32 FTerminalBuffer::GetRingIndex(int32 AbsoluteRow) const
{
	const int32 MaxRows = Rows.Num();
	if (MaxRows == 0)
	{
		return 0;
	}
	return (RingStart + AbsoluteRow) % MaxRows;
}

void FTerminalBuffer::EnsureRow(int32 RingIndex)
{
	if (Rows.IsValidIndex(RingIndex) && Rows[RingIndex].Num() != Columns)
	{
		Rows[RingIndex].SetNum(Columns);
		for (FTerminalCell& Cell : Rows[RingIndex])
		{
			Cell = DefaultCell;
		}
	}
	else if (!Rows.IsValidIndex(RingIndex))
	{
		// Should not happen with proper initialization, but defensive.
		Rows.SetNum(RingIndex + 1);
		Rows[RingIndex].SetNum(Columns);
		for (FTerminalCell& Cell : Rows[RingIndex])
		{
			Cell = DefaultCell;
		}
	}
}
