// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalColorScheme.h"

#include "TerminalUtilities.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

FTerminalColorScheme FTerminalColorScheme::MakeDefault()
{
	FTerminalColorScheme Scheme;
	Scheme.Name = TEXT("Default");
	Scheme.DefaultForeground = TerminalUtilities::ParseHexColor(TEXT("#D4D4D4"));
	Scheme.DefaultBackground = TerminalUtilities::ParseHexColor(TEXT("#1E1E1E"));
	Scheme.CursorColor = TerminalUtilities::ParseHexColor(TEXT("#AEAFAD"));
	Scheme.SelectionColor = TerminalUtilities::ParseHexColor(TEXT("#264F78"));
	Scheme.Palette = {
		TerminalUtilities::ParseHexColor(TEXT("#000000")), TerminalUtilities::ParseHexColor(TEXT("#CD3131")),
		TerminalUtilities::ParseHexColor(TEXT("#0DBC79")), TerminalUtilities::ParseHexColor(TEXT("#E5E510")),
		TerminalUtilities::ParseHexColor(TEXT("#2472C8")), TerminalUtilities::ParseHexColor(TEXT("#BC3FBC")),
		TerminalUtilities::ParseHexColor(TEXT("#11A8CD")), TerminalUtilities::ParseHexColor(TEXT("#E5E5E5")),
		TerminalUtilities::ParseHexColor(TEXT("#666666")), TerminalUtilities::ParseHexColor(TEXT("#F14C4C")),
		TerminalUtilities::ParseHexColor(TEXT("#23D18B")), TerminalUtilities::ParseHexColor(TEXT("#F5F543")),
		TerminalUtilities::ParseHexColor(TEXT("#3B8EEA")), TerminalUtilities::ParseHexColor(TEXT("#D670D6")),
		TerminalUtilities::ParseHexColor(TEXT("#29B8DB")), TerminalUtilities::ParseHexColor(TEXT("#FFFFFF")),
	};
	return Scheme;
}

bool FTerminalColorScheme::FromJSON(const FString& JSONString, FTerminalColorScheme& OutScheme)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JSONString);
	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	if (!JsonObject->HasField(TEXT("name")) ||
		!JsonObject->HasField(TEXT("defaultForeground")) ||
		!JsonObject->HasField(TEXT("defaultBackground")) ||
		!JsonObject->HasField(TEXT("cursorColor")) ||
		!JsonObject->HasField(TEXT("selectionColor")))
	{
		return false;
	}

	if (!JsonObject->TryGetStringField(TEXT("name"), OutScheme.Name))
	{
		return false;
	}

	FString ForegroundHex, BackgroundHex, CursorHex, SelectionHex;
	if (!JsonObject->TryGetStringField(TEXT("defaultForeground"), ForegroundHex) ||
		!JsonObject->TryGetStringField(TEXT("defaultBackground"), BackgroundHex) ||
		!JsonObject->TryGetStringField(TEXT("cursorColor"), CursorHex) ||
		!JsonObject->TryGetStringField(TEXT("selectionColor"), SelectionHex))
	{
		return false;
	}

	OutScheme.DefaultForeground = TerminalUtilities::ParseHexColor(ForegroundHex);
	OutScheme.DefaultBackground = TerminalUtilities::ParseHexColor(BackgroundHex);
	OutScheme.CursorColor = TerminalUtilities::ParseHexColor(CursorHex);
	OutScheme.SelectionColor = TerminalUtilities::ParseHexColor(SelectionHex);

	OutScheme.Palette.Reset();
	const TArray<TSharedPtr<FJsonValue>>* PaletteArray = nullptr;
	if (JsonObject->TryGetArrayField(TEXT("palette"), PaletteArray) && PaletteArray)
	{
		for (const TSharedPtr<FJsonValue>& Value : *PaletteArray)
		{
			OutScheme.Palette.Add(TerminalUtilities::ParseHexColor(Value->AsString()));
		}
	}

	// Ensure palette has exactly 16 entries.
	while (OutScheme.Palette.Num() < 16)
	{
		OutScheme.Palette.Add(FLinearColor::White);
	}
	if (OutScheme.Palette.Num() > 16)
	{
		OutScheme.Palette.SetNum(16);
	}

	return true;
}
