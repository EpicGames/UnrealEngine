// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalKeyTranslator.h"

#include "InputCoreTypes.h"
#include "Input/Events.h"

namespace UE::Terminal
{
	namespace
	{
		struct FEffectiveModifiers
		{
			bool bShift;
			bool bCtrl;
			bool bAlt;
		};

		bool IsAltGrDown(const FKeyEvent& KeyEvent)
		{
			return KeyEvent.IsLeftControlDown() && KeyEvent.IsRightAltDown();
		}

		FEffectiveModifiers GetEffectiveModifiers(const FKeyEvent& KeyEvent)
		{
			FEffectiveModifiers Modifiers;
			Modifiers.bShift = KeyEvent.IsShiftDown();
			if (IsAltGrDown(KeyEvent))
			{
				Modifiers.bCtrl = false;
				Modifiers.bAlt = false;
			}
			else
			{
				Modifiers.bCtrl = KeyEvent.IsControlDown();
				Modifiers.bAlt = KeyEvent.IsAltDown();
			}
			return Modifiers;
		}

		uint8 ComputeModifierParam(const FEffectiveModifiers& Modifiers)
		{
			return static_cast<uint8>(1 + (Modifiers.bShift ? 1 : 0) + (Modifiers.bAlt ? 2 : 0) + (Modifiers.bCtrl ? 4 : 0));
		}

		void AppendAnsi(TArray<uint8>& Out, const char* Bytes)
		{
			for (const char* It = Bytes; *It; ++It)
			{
				Out.Add(static_cast<uint8>(*It));
			}
		}

		/** Build an xterm CSI sequence: `ESC [ <Prefix> ; <ModParam> <Final>` when modified,
		 *  or `ESC [ <Prefix> <Final>` when unmodified. Omits the leading `<Prefix>` when empty
		 *  and the key only has a final letter (e.g., arrows). */
		TArray<uint8> MakeCsiModified(const char* Prefix, uint8 ModParam, char Final)
		{
			TArray<uint8> Out;
			Out.Add(0x1B);
			Out.Add('[');
			if (Prefix && *Prefix)
			{
				AppendAnsi(Out, Prefix);
			}
			else
			{
				Out.Add('1');
			}
			if (ModParam > 1)
			{
				Out.Add(';');
				Out.Add('0' + ModParam);
			}
			Out.Add(static_cast<uint8>(Final));
			return Out;
		}

		/** Tilde-terminated CSI (Insert, Delete, PageUp/Down, F5..F12): `ESC [ <N> ~` or `ESC [ <N> ; <mod> ~`. */
		TArray<uint8> MakeCsiTildeModified(const char* NumberPrefix, uint8 ModParam)
		{
			TArray<uint8> Out;
			Out.Add(0x1B);
			Out.Add('[');
			AppendAnsi(Out, NumberPrefix);
			if (ModParam > 1)
			{
				Out.Add(';');
				Out.Add('0' + ModParam);
			}
			Out.Add('~');
			return Out;
		}

		TArray<uint8> MakeSs3(char Final)
		{
			return { 0x1B, 'O', static_cast<uint8>(Final) };
		}

		/** Translate unmodified cursor / Home / End keys honoring DECCKM (bApplicationCursorKeys). */
		TArray<uint8> TranslateCursorUnmodified(const FKey& Key, bool bApplicationCursorKeys)
		{
			char Final = 0;
			if (Key == EKeys::Up) Final = 'A';
			else if (Key == EKeys::Down) Final = 'B';
			else if (Key == EKeys::Right) Final = 'C';
			else if (Key == EKeys::Left) Final = 'D';
			else if (Key == EKeys::Home) Final = 'H';
			else if (Key == EKeys::End) Final = 'F';
			else return {};

			if (bApplicationCursorKeys)
			{
				return MakeSs3(Final);
			}
			return { 0x1B, '[', static_cast<uint8>(Final) };
		}

		char CursorFinal(const FKey& Key)
		{
			if (Key == EKeys::Up) return 'A';
			if (Key == EKeys::Down) return 'B';
			if (Key == EKeys::Right) return 'C';
			if (Key == EKeys::Left) return 'D';
			if (Key == EKeys::Home) return 'H';
			if (Key == EKeys::End) return 'F';
			return 0;
		}

		char F1ToF4Final(const FKey& Key)
		{
			if (Key == EKeys::F1) return 'P';
			if (Key == EKeys::F2) return 'Q';
			if (Key == EKeys::F3) return 'R';
			if (Key == EKeys::F4) return 'S';
			return 0;
		}

		const char* F5ToF12Number(const FKey& Key)
		{
			if (Key == EKeys::F5) return "15";
			if (Key == EKeys::F6) return "17";
			if (Key == EKeys::F7) return "18";
			if (Key == EKeys::F8) return "19";
			if (Key == EKeys::F9) return "20";
			if (Key == EKeys::F10) return "21";
			if (Key == EKeys::F11) return "23";
			if (Key == EKeys::F12) return "24";
			return nullptr;
		}

		/** Returns the ASCII letter ('A'..'Z') for a letter-key FKey, or 0 otherwise. */
		char LetterFromKey(const FKey& Key)
		{
			const FString Name = Key.ToString();
			if (Name.Len() == 1)
			{
				const TCHAR Character = Name[0];
				if (Character >= TEXT('A') && Character <= TEXT('Z'))
				{
					return static_cast<char>(Character);
				}
			}
			return 0;
		}

		/** Returns the Ctrl-folded byte for letters and the supported symbols, or -1 when none applies. */
		int32 CtrlByteForKey(const FKey& Key, bool bShift)
		{
			if (const char Letter = LetterFromKey(Key))
			{
				// Ctrl+A..Z -> 0x01..0x1A via the `ch & 0x1F` mask.
				return Letter & 0x1F;
			}
			if (Key == EKeys::SpaceBar)
			{
				return 0x00;
			}
			if (Key == EKeys::LeftBracket)
			{
				return 0x1B;
			}
			if (Key == EKeys::Backslash)
			{
				return 0x1C;
			}
			if (Key == EKeys::RightBracket)
			{
				return 0x1D;
			}
			if (Key == EKeys::Slash)
			{
				// Ctrl+/ -> 0x1F, Ctrl+? (Ctrl+Shift+/ on US layout) -> 0x7F.
				return bShift ? 0x7F : 0x1F;
			}
			return -1;
		}
	}

	TArray<uint8> TranslateKeyToBytes(const FKeyEvent& KeyEvent, const FKeyTranslationOptions& Options)
	{
		const FKey Key = KeyEvent.GetKey();
		const FEffectiveModifiers Modifiers = GetEffectiveModifiers(KeyEvent);
		const uint8 ModParam = ComputeModifierParam(Modifiers);
		const bool bModified = ModParam > 1;

		// Pure modifier presses produce no input.
		if (Key == EKeys::LeftShift || Key == EKeys::RightShift
			|| Key == EKeys::LeftControl || Key == EKeys::RightControl
			|| Key == EKeys::LeftAlt || Key == EKeys::RightAlt
			|| Key == EKeys::LeftCommand || Key == EKeys::RightCommand)
		{
			return {};
		}

		// Cursor and navigation keys.
		if (Key == EKeys::Up || Key == EKeys::Down || Key == EKeys::Left || Key == EKeys::Right
			|| Key == EKeys::Home || Key == EKeys::End)
		{
			if (!bModified)
			{
				return TranslateCursorUnmodified(Key, Options.bApplicationCursorKeys);
			}
			return MakeCsiModified(nullptr, ModParam, CursorFinal(Key));
		}

		if (Key == EKeys::Insert)
		{
			return MakeCsiTildeModified("2", ModParam);
		}
		if (Key == EKeys::Delete)
		{
			return MakeCsiTildeModified("3", ModParam);
		}
		if (Key == EKeys::PageUp)
		{
			return MakeCsiTildeModified("5", ModParam);
		}
		if (Key == EKeys::PageDown)
		{
			return MakeCsiTildeModified("6", ModParam);
		}

		// Function keys.
		if (const char F1to4Final = F1ToF4Final(Key))
		{
			if (!bModified)
			{
				return MakeSs3(F1to4Final);
			}
			return MakeCsiModified(nullptr, ModParam, F1to4Final);
		}
		if (const char* F5to12Number = F5ToF12Number(Key))
		{
			return MakeCsiTildeModified(F5to12Number, ModParam);
		}

		// Tab / Shift+Tab.
		if (Key == EKeys::Tab)
		{
			if (Modifiers.bShift && !Modifiers.bCtrl && !Modifiers.bAlt)
			{
				return { 0x1B, '[', 'Z' };
			}
			return { 0x09 };
		}

		if (Key == EKeys::Enter)
		{
			return { 0x0D };
		}

		if (Key == EKeys::Escape)
		{
			return { 0x1B };
		}

		if (Key == EKeys::BackSpace)
		{
			if (Modifiers.bCtrl && !Modifiers.bAlt)
			{
				return { 0x08 };
			}
			if (Modifiers.bAlt && !Modifiers.bCtrl)
			{
				return { 0x1B, 0x7F };
			}
			return { 0x7F };
		}

		// Text/symbol keys. Compute the "inner" byte (pre Alt-prefix) from Ctrl or Alt state.
		//
		// xterm meta convention: when Alt is held (and not AltGr), prefix ESC regardless of Ctrl.
		// Ctrl+letter folds via `ch & 0x1F`; plain Alt+letter emits the lowercase letter.
		int32 InnerByte = -1;
		if (Modifiers.bCtrl)
		{
			InnerByte = CtrlByteForKey(Key, Modifiers.bShift);
		}
		else if (Modifiers.bAlt)
		{
			if (const char Letter = LetterFromKey(Key))
			{
				InnerByte = static_cast<uint8>(Letter - 'A' + 'a');
			}
		}

		if (InnerByte < 0)
		{
			return {};
		}

		if (Modifiers.bAlt)
		{
			return { 0x1B, static_cast<uint8>(InnerByte) };
		}
		return { static_cast<uint8>(InnerByte) };
	}
}
