// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalModule.h"

#include "ITerminalSession.h"
#include "STerminal.h"
#include "TerminalSettings.h"
#include "TerminalSubsystem.h"
#include "Editor.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

DEFINE_LOG_CATEGORY(LogTerminal);

IMPLEMENT_MODULE(FTerminalModule, Terminal)

#define LOCTEXT_NAMESPACE "FTerminalModule"

static const FName TerminalTabName("Terminal");

void FTerminalModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TerminalTabName, FOnSpawnTab::CreateRaw(this, &FTerminalModule::SpawnTerminalTab))
		.SetDisplayName(LOCTEXT("Terminal", "Terminal"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([](const FTabId&)
		{
			return TSharedPtr<SDockTab>();
		}));

	// Commandlets and other non-interactive contexts have no editor frame to close, and MainFrame may not be loadable.
	if (!IsRunningCommandlet())
	{
		if (IMainFrameModule* MainFrame = FModuleManager::LoadModulePtr<IMainFrameModule>("MainFrame"))
		{
			CanCloseEditorHandle = MainFrame->RegisterCanCloseEditor(IMainFrameModule::FMainFrameCanCloseEditor::CreateRaw(this, &FTerminalModule::HandleCanCloseEditor));
		}
	}
}

void FTerminalModule::ShutdownModule()
{
	if (CanCloseEditorHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::GetModuleChecked<IMainFrameModule>("MainFrame");
		MainFrame.UnregisterCanCloseEditor(CanCloseEditorHandle);
	}
	CanCloseEditorHandle.Reset();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TerminalTabName);
}

TSharedRef<SDockTab> FTerminalModule::SpawnTerminalTab(const FSpawnTabArgs& SpawnTabArgs)
{
	TSharedRef<SScrollBar> Scrollbar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.AlwaysShowScrollbar(true)
		.Thickness(FVector2D(6.0f, 6.0f));

	TSharedRef<STerminal> TerminalWidget = SNew(STerminal)
		.ExternalScrollbar(Scrollbar);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				TerminalWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				Scrollbar
			]
		];

	// Auto-focus the terminal widget when the user clicks on the tab header so it
	// is immediately ready for keyboard input.  Only respond to UserClickedOnTab  -
	// other causes (programmatic activation, layout restoration on startup) must be
	// ignored because SetKeyboardFocus itself triggers tab activation, which might
	// create a feedback loop.
	TWeakPtr<STerminal> WeakTerminal = TerminalWidget;
	DockTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateLambda(
		[WeakTerminal](TSharedRef<SDockTab>, ETabActivationCause ActivationCause)
		{
			if (ActivationCause != ETabActivationCause::UserClickedOnTab)
			{
				return;
			}

			if (TSharedPtr<STerminal> Terminal = WeakTerminal.Pin())
			{
				FSlateApplication::Get().SetKeyboardFocus(Terminal);
			}
		}));

	return DockTab;
}

bool FTerminalModule::HandleCanCloseEditor()
{
	const UTerminalSettings* Settings = GetDefault<UTerminalSettings>();
	if (!Settings || !Settings->bPreventCloseDuringActivity)
	{
		return true;
	}

	const double Now = FApp::GetCurrentTime();
	const double Window = static_cast<double>(Settings->ActivityTimeoutSeconds);
	int32 ActiveCount = 0;

	const TArray<TSharedRef<STerminal>> Terminals = STerminal::GetAllInstances();
	for (const TSharedRef<STerminal>& Terminal : Terminals)
	{
		// Don't gate on IsSessionRunning() - a process that just exited may have flushed its
		// final output milliseconds ago, and the user still cares about that data.
		const double LastOutputTime = Terminal->GetLastOutputTime();
		if (LastOutputTime > 0.0 && (Now - LastOutputTime) < Window)
		{
			++ActiveCount;
		}
	}

	UE_LOGF(LogTerminal, Display, "HandleCanCloseEditor: Terminals=%d, Active=%d, Window=%.1fs.", Terminals.Num(), ActiveCount, Window);

	if (ActiveCount == 0)
	{
		return true;
	}

	const FText Message = FText::Format(
		LOCTEXT("TerminalActivityCloseConfirm", "{0} {0}|plural(one=terminal is,other=terminals are) currently producing output. Close the editor anyway?"),
		FText::AsNumber(ActiveCount));

	const EAppReturnType::Type Response = FMessageDialog::Open(
		EAppMsgCategory::Warning,
		EAppMsgType::YesNo,
		Message,
		LOCTEXT("TerminalActivityCloseTitle", "Terminal Activity"));

	return Response == EAppReturnType::Yes;
}

#undef LOCTEXT_NAMESPACE
