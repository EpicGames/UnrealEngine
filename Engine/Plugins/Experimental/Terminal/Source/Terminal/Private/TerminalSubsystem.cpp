// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalSubsystem.h"

#include "ConPTYSession.h"
#include "TerminalSettings.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

void UTerminalSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	DefaultScheme = FTerminalColorScheme::MakeDefault();
	ReloadColorSchemes();
}

void UTerminalSubsystem::Deinitialize()
{
	ColorSchemes.Empty();

	Super::Deinitialize();
}

FTerminalColorScheme UTerminalSubsystem::GetActiveColorScheme() const
{
	const UTerminalSettings* Settings = GetDefault<UTerminalSettings>();
	const FString SchemeName = Settings ? Settings->ColorSchemeName : TEXT("Default");
	return GetColorScheme(SchemeName);
}

FTerminalColorScheme UTerminalSubsystem::GetColorScheme(const FString& Name) const
{
	if (const FTerminalColorScheme* Found = ColorSchemes.Find(Name))
	{
		return *Found;
	}

	UE_LOGF(LogTerminal, Warning, "Color scheme '%ls' not found, using default.", *Name);
	return DefaultScheme;
}

void UTerminalSubsystem::ReloadColorSchemes()
{
	ColorSchemes.Empty();

	// Always register the hardcoded default.
	ColorSchemes.Add(DefaultScheme.Name, DefaultScheme);

	// Find the plugin's Config/ColorSchemes/ directory.
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	if (!Plugin.IsValid())
	{
		return;
	}

	const FString ColorSchemesDirectory = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Config"), TEXT("ColorSchemes"));
	if (!FPaths::DirectoryExists(ColorSchemesDirectory))
	{
		return;
	}

	// Scan for JSON files.
	TArray<FString> JSONFiles;
	IFileManager::Get().FindFiles(JSONFiles, *FPaths::Combine(ColorSchemesDirectory, TEXT("*.json")), true, false);

	for (const FString& FileName : JSONFiles)
	{
		const FString FilePath = FPaths::Combine(ColorSchemesDirectory, FileName);
		FString JSONContent;
		if (FFileHelper::LoadFileToString(JSONContent, *FilePath))
		{
			FTerminalColorScheme Scheme;
			if (FTerminalColorScheme::FromJSON(JSONContent, Scheme) && !Scheme.Name.IsEmpty())
			{
				ColorSchemes.Add(Scheme.Name, Scheme);
				UE_LOGF(LogTerminal, Verbose, "Loaded color scheme: %ls", *Scheme.Name);
			}
			else
			{
				UE_LOGF(LogTerminal, Warning, "Failed to parse color scheme file: %ls", *FilePath);
			}
		}
	}

	UE_LOGF(LogTerminal, Display, "Loaded %d terminal color scheme(s).", ColorSchemes.Num());
}
