// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerminalUtilities.h"

#include "ITerminalSession.h"
#include "Misc/Paths.h"

bool TerminalUtilities::IsMonospaceSafe(TCHAR Character)
{
	return Character >= 0x20 && Character <= 0x7E;
}

FString TerminalUtilities::ResolveSystemFontPath(const FString& FontFamily)
{
	// Reject font family names that contain path separators or relative path
	// components to prevent path traversal when constructing candidate paths.
	if (FontFamily.IsEmpty()
		|| FontFamily.Contains(TEXT("/"))
		|| FontFamily.Contains(TEXT("\\"))
		|| FontFamily.Contains(TEXT(".."))
		|| FontFamily.Contains(TEXT(":")))
	{
		UE_LOGF(LogTerminal, Warning, "Rejecting font family name with invalid characters: '%ls'", *FontFamily);
		return FString();
	}

#if PLATFORM_WINDOWS
	const FString WindowsFontsDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("WINDIR")) / TEXT("Fonts");
	const FString FilePath = WindowsFontsDirectory / (FontFamily + TEXT(".ttf"));
	return FPaths::FileExists(FilePath) ? FilePath : FString();
#elif PLATFORM_MAC
	static const TCHAR* SearchDirectories[] = {
		TEXT("/System/Library/Fonts"),
		TEXT("/Library/Fonts"),
	};

	// Also search ~/Library/Fonts.
	const FString HomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));
	const FString UserFontsDirectory = HomeDirectory / TEXT("Library/Fonts");

	TArray<const TCHAR*, TFixedAllocator<3>> AllDirectories;
	for (const TCHAR* Directory : SearchDirectories)
	{
		AllDirectories.Add(Directory);
	}
	if (!HomeDirectory.IsEmpty())
	{
		AllDirectories.Add(*UserFontsDirectory);
	}

	for (const TCHAR* Directory : AllDirectories)
	{
		const FString FilePath = FString(Directory) / (FontFamily + TEXT(".ttf"));
		if (FPaths::FileExists(FilePath))
		{
			return FilePath;
		}
		// macOS also uses .ttc (TrueType Collection) files.
		const FString CollectionPath = FString(Directory) / (FontFamily + TEXT(".ttc"));
		if (FPaths::FileExists(CollectionPath))
		{
			return CollectionPath;
		}
	}
	return FString();
#elif PLATFORM_LINUX
	// Search common Linux font directories. Some fonts live in subdirectories
	// (e.g. /usr/share/fonts/truetype/dejavu/DejaVuSansMono.ttf), so we try
	// both flat and one level of common subdirectory structures.
	const FString HomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));

	TArray<FString> SearchDirectories;
	SearchDirectories.Add(TEXT("/usr/share/fonts"));
	SearchDirectories.Add(TEXT("/usr/share/fonts/truetype"));
	SearchDirectories.Add(TEXT("/usr/local/share/fonts"));
	if (!HomeDirectory.IsEmpty())
	{
		SearchDirectories.Add(HomeDirectory / TEXT(".local/share/fonts"));
	}

	const FString FamilyLower = FontFamily.ToLower();

	for (const FString& Directory : SearchDirectories)
	{
		const FString FilePath = Directory / (FontFamily + TEXT(".ttf"));
		if (FPaths::FileExists(FilePath))
		{
			return FilePath;
		}

		// Many distros organize fonts into subdirectories named after the family.
		// Try common patterns: truetype/<family>/, <family>/, TTF/.
		static const TCHAR* SubDirectoryPatterns[] = { TEXT("truetype"), TEXT("TTF") };
		for (const TCHAR* SubDirectory : SubDirectoryPatterns)
		{
			const FString SubPath = Directory / SubDirectory / (FontFamily + TEXT(".ttf"));
			if (FPaths::FileExists(SubPath))
			{
				return SubPath;
			}
		}

		// Also try: <directory>/<familyLower>/<FontFamily>.ttf (e.g. dejavu/DejaVuSansMono.ttf).
		const FString FamilySubPath = Directory / FamilyLower / (FontFamily + TEXT(".ttf"));
		if (FPaths::FileExists(FamilySubPath))
		{
			return FamilySubPath;
		}
	}
	return FString();
#else
	return FString();
#endif
}

TArrayView<const TCHAR* const> TerminalUtilities::GetFallbackFontNames()
{
#if PLATFORM_WINDOWS
	static const TCHAR* Names[] = { TEXT("CascadiaMono"), TEXT("CascadiaCode"), TEXT("consola"), TEXT("cour") };
#elif PLATFORM_MAC
	static const TCHAR* Names[] = { TEXT("SFMono-Regular"), TEXT("Menlo-Regular"), TEXT("Monaco") };
#else
	static const TCHAR* Names[] = { TEXT("DejaVuSansMono"), TEXT("LiberationMono"), TEXT("UbuntuMono"), TEXT("DroidSansMono") };
#endif
	return MakeArrayView(Names);
}

TArrayView<const TCHAR* const> TerminalUtilities::GetSymbolFontNames()
{
#if PLATFORM_WINDOWS
	static const TCHAR* Names[] = { TEXT("seguisym") };
#elif PLATFORM_MAC
	static const TCHAR* Names[] = { TEXT("Apple Symbols") };
#else
	static const TCHAR* Names[] = { TEXT("DejaVuSans"), TEXT("NotoSansSymbols2-Regular") };
#endif
	return MakeArrayView(Names);
}

TArrayView<const TCHAR* const> TerminalUtilities::GetEmojiFontNames()
{
#if PLATFORM_WINDOWS
	static const TCHAR* Names[] = { TEXT("seguiemj") };
#elif PLATFORM_MAC
	static const TCHAR* Names[] = { TEXT("Apple Color Emoji") };
#else
	static const TCHAR* Names[] = { TEXT("NotoColorEmoji") };
#endif
	return MakeArrayView(Names);
}

FString TerminalUtilities::GetEngineFallbackFontPath()
{
	return FPaths::EngineContentDir() / TEXT("Slate/Fonts/DroidSansMono.ttf");
}

FLinearColor TerminalUtilities::ParseHexColor(const FString& HexString)
{
	FColor Color = FColor::White;
	if (HexString.StartsWith(TEXT("#")) && HexString.Len() == 7)
	{
		Color = FColor::FromHex(HexString);
	}
	else if (HexString.Len() == 6)
	{
		Color = FColor::FromHex(HexString);
	}
	return FLinearColor(Color);
}
