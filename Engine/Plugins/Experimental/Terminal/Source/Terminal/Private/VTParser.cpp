// Copyright Epic Games, Inc. All Rights Reserved.

#include "VTParser.h"

#include "TerminalBuffer.h"

// -- Sequence accumulator caps --
// These limits prevent unbounded memory growth from pathological input.
// Input exceeding a cap is silently discarded; the sequence still terminates normally.

/**
 * Max characters in an OSC payload. OSC 52 (clipboard) carries base64-encoded data that can
 * include images  - a 1920x1080 screenshot at 4 bytes/pixel is ~8MB base64. xterm has no hard
 * limit; xterm.js caps at 10MB; tmux at ~32KB. Allowing up to 16mb.
 */
static constexpr int32 MaxOSCPayloadLength = 16 * 1024 * 1024;

/**
 * Max digits per individual CSI parameter. Windows Console limits values to int16 (~5 digits);
 * xterm.js allows up to 2^31-1 (~10 digits).
 */
static constexpr int32 MaxCSIParamDigits = 16;

/**
 * Max number of semicolon-separated CSI parameters. DEC VT510 allows 16; xterm uses ~30
 * (NPARAM); xterm.js allows 32.
 */
static constexpr int32 MaxCSIParamCount = 32;

/**
 * Max intermediate bytes (0x20-0x2F) in a CSI or escape sequence. ECMA-48 specifies at most
 * one intermediate byte per sequence in practice; two is the observed maximum. 4 is defensive.
 */
static constexpr int32 MaxIntermediateChars = 4;

/** Default ANSI 16-color palette values. */
static const FColor DefaultAnsiPalette[16] = {
	FColor(0, 0, 0),       FColor(205, 49, 49),    FColor(13, 188, 121),  FColor(229, 229, 16),
	FColor(36, 114, 200),   FColor(188, 63, 188),   FColor(17, 168, 205),  FColor(229, 229, 229),
	FColor(102, 102, 102),  FColor(241, 76, 76),    FColor(35, 209, 139),  FColor(245, 245, 67),
	FColor(59, 142, 234),   FColor(214, 112, 214),  FColor(41, 184, 219),  FColor(255, 255, 255),
};

FVTParser::FVTParser()
{
	Reset();
}

void FVTParser::SetBuffer(FTerminalBuffer* InBuffer)
{
	Buffer = InBuffer;
}

void FVTParser::Reset()
{
	State = EParserState::Ground;
	CurrentForeground = FColor(212, 212, 212);
	CurrentBackground = FColor(30, 30, 30);
	CurrentAttributes = ETerminalAttribute::None;
	FMemory::Memcpy(AnsiPalette, DefaultAnsiPalette, sizeof(AnsiPalette));
	ScrollRegionTop = -1;
	ScrollRegionBottom = -1;
	bAutoWrap = true;
	bCursorVisible = true;
	bApplicationCursorKeys = false;
	bBracketedPaste = false;
	WindowTitle.Empty();
	ResponseBuffer.Reset();
	ClearSequenceState();
	EscapeIntermediateChars.Empty();
	OSCPayload.Empty();
	UTF8Codepoint = 0;
	UTF8Remaining = 0;
	InitializeTabStops();
}

FString FVTParser::GetDebugStateName() const
{
	switch (State)
	{
	case EParserState::Ground:              return TEXT("Ground");
	case EParserState::Escape:              return TEXT("Escape");
	case EParserState::EscapeIntermediate:  return TEXT("EscapeIntermediate");
	case EParserState::CSIEntry:            return TEXT("CSIEntry");
	case EParserState::CSIParam:            return TEXT("CSIParam");
	case EParserState::CSIIntermediate:     return TEXT("CSIIntermediate");
	case EParserState::CSIIgnore:           return TEXT("CSIIgnore");
	case EParserState::OSCString:           return TEXT("OSCString");
	case EParserState::DCSEntry:            return TEXT("DCSEntry");
	case EParserState::DCSParam:            return TEXT("DCSParam");
	case EParserState::DCSIntermediate:     return TEXT("DCSIntermediate");
	case EParserState::DCSPassthrough:      return TEXT("DCSPassthrough");
	case EParserState::DCSIgnore:           return TEXT("DCSIgnore");
	case EParserState::SOSPMAPCString:      return TEXT("SOSPMAPCString");
	default:                                return TEXT("Unknown");
	}
}

void FVTParser::ClearSequenceState()
{
	CSIParams.Reset();
	CSIIntermediateChars.Empty();
	CurrentParam.Empty();
	CSIPrivatePrefix = 0;
}

bool FVTParser::TryExecuteC0(uint32 Codepoint)
{
	// C0 controls (0x00-0x1F) that should be executed even mid-sequence.
	// CAN (0x18), SUB (0x1A), ESC (0x1B) are handled as "anywhere" transitions
	// in ProcessCodepoint before reaching this function.
	if (!Buffer)
	{
		return Codepoint < 0x20;
	}

	switch (Codepoint)
	{
	case 0x07: // BEL
		return true;
	case 0x08: // BS  - backspace
		if (Buffer->Cursor.Column > 0)
		{
			Buffer->Cursor.Column--;
		}
		return true;
	case 0x09: // HT  - horizontal tab
	{
		const int32 Columns = Buffer->GetColumns();
		int32 NextTab = Buffer->Cursor.Column + 1;
		while (NextTab < Columns)
		{
			if (NextTab < TabStops.Num() && TabStops[NextTab])
			{
				break;
			}
			NextTab++;
		}
		Buffer->Cursor.Column = FMath::Min(NextTab, Columns - 1);
		return true;
	}
	case 0x0A: // LF  - line feed
	case 0x0B: // VT  - vertical tab (treated as LF)
	case 0x0C: // FF  - form feed (treated as LF)
		LineFeed();
		return true;
	case 0x0D: // CR  - carriage return
		Buffer->Cursor.Column = 0;
		return true;
	default:
		// Other C0 controls (NUL, SOH, etc.)  - ignore.
		return Codepoint < 0x20;
	}
}

void FVTParser::Parse(const uint8* Data, int32 Length)
{
	if (!Buffer)
	{
		return;
	}

	for (int32 Index = 0; Index < Length; ++Index)
	{
		const uint8 Byte = Data[Index];

		// UTF-8 decoding.
		if (UTF8Remaining > 0)
		{
			if ((Byte & 0xC0) == 0x80)
			{
				UTF8Codepoint = (UTF8Codepoint << 6) | (Byte & 0x3F);
				UTF8Remaining--;
				if (UTF8Remaining == 0)
				{
					// Reject ill-formed UTF-8: overlong encodings, surrogates, out-of-range.
					const bool bOverlong = UTF8Codepoint < UTF8MinCodepoint;
					const bool bSurrogate = UTF8Codepoint >= 0xD800 && UTF8Codepoint <= 0xDFFF;
					const bool bOutOfRange = UTF8Codepoint > 0x10FFFF;

					if (!bOverlong && !bSurrogate && !bOutOfRange)
					{
						ProcessCodepoint(UTF8Codepoint);
					}
				}
			}
			else
			{
				// Invalid continuation byte  - reset and process as new byte.
				UTF8Remaining = 0;
				UTF8Codepoint = 0;
				// Fall through to process this byte as a new sequence start.
				goto NewByte;
			}
			continue;
		}

	NewByte:
		if (Byte < 0x80)
		{
			ProcessCodepoint(Byte);
		}
		else if ((Byte & 0xE0) == 0xC0)
		{
			UTF8Codepoint = Byte & 0x1F;
			UTF8Remaining = 1;
			UTF8MinCodepoint = 0x80;
		}
		else if ((Byte & 0xF0) == 0xE0)
		{
			UTF8Codepoint = Byte & 0x0F;
			UTF8Remaining = 2;
			UTF8MinCodepoint = 0x800;
		}
		else if ((Byte & 0xF8) == 0xF0)
		{
			UTF8Codepoint = Byte & 0x07;
			UTF8Remaining = 3;
			UTF8MinCodepoint = 0x10000;
		}
		else
		{
			// Invalid byte  - skip.
		}
	}
}

void FVTParser::ProcessCodepoint(uint32 Codepoint)
{
	// -- "Anywhere" transitions  - highest priority, override all states --

	if (Codepoint == 0x1B) // ESC  - abort any sequence, begin new escape
	{
		ClearSequenceState();
		EscapeIntermediateChars.Empty();
		State = EParserState::Escape;
		return;
	}

	if (Codepoint == 0x18 || Codepoint == 0x1A) // CAN, SUB  - abort to ground
	{
		ClearSequenceState();
		OSCPayload.Empty();
		State = EParserState::Ground;
		return;
	}

	// 8-bit C1 controls (U+0080-U+009F)  - "anywhere" transitions.
	// In a UTF-8 terminal these arrive as decoded codepoints, not raw bytes.
	if (Codepoint >= 0x80 && Codepoint <= 0x9F)
	{
		switch (Codepoint)
		{
		case 0x90: // DCS
			ClearSequenceState();
			State = EParserState::DCSEntry;
			return;
		case 0x9B: // CSI
			ClearSequenceState();
			State = EParserState::CSIEntry;
			return;
		case 0x9D: // OSC
			OSCPayload.Empty();
			State = EParserState::OSCString;
			return;
		case 0x9C: // ST  - string terminator
			State = EParserState::Ground;
			return;
		case 0x98: // SOS
		case 0x9E: // PM
		case 0x9F: // APC
			State = EParserState::SOSPMAPCString;
			return;
		default:
			// Other 8-bit C1 controls  - execute (treat as no-op) and go to ground.
			State = EParserState::Ground;
			return;
		}
	}

	// -- State-specific dispatch --

	switch (State)
	{
	case EParserState::Ground:            HandleGround(Codepoint); break;
	case EParserState::Escape:            HandleEscape(Codepoint); break;
	case EParserState::EscapeIntermediate: HandleEscapeIntermediate(Codepoint); break;
	case EParserState::CSIEntry:          HandleCSIEntry(Codepoint); break;
	case EParserState::CSIParam:          HandleCSIParam(Codepoint); break;
	case EParserState::CSIIntermediate:   HandleCSIIntermediate(Codepoint); break;
	case EParserState::CSIIgnore:         HandleCSIIgnore(Codepoint); break;
	case EParserState::OSCString:         HandleOSCString(Codepoint); break;
	case EParserState::DCSEntry:          HandleDCSEntry(Codepoint); break;
	case EParserState::DCSParam:          HandleDCSParam(Codepoint); break;
	case EParserState::DCSIntermediate:   HandleDCSIntermediate(Codepoint); break;
	case EParserState::DCSPassthrough:    HandleDCSPassthrough(Codepoint); break;
	case EParserState::DCSIgnore:         HandleDCSIgnore(Codepoint); break;
	case EParserState::SOSPMAPCString:    HandleSOSPMAPCString(Codepoint); break;
	}
}

// -- Ground --

void FVTParser::HandleGround(uint32 Codepoint)
{
	// C0 controls (except ESC/CAN/SUB which are handled in ProcessCodepoint).
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// DEL  - ignore in ground.
	if (Codepoint == 0x7F)
	{
		return;
	}

	// Printable characters (0x20+).
	if (IsZeroWidthCodepoint(Codepoint))
	{
		return;
	}
	// TCHAR is 16-bit on Windows; codepoints above U+FFFF cannot be represented
	// in a single TCHAR without surrogate pairs. Replace with U+FFFD to avoid truncation.
	if (Codepoint > 0xFFFF)
	{
		PutCharacter(static_cast<TCHAR>(0xFFFD));
	}
	else
	{
		PutCharacter(static_cast<TCHAR>(Codepoint));
	}
}

// -- Escape --

void FVTParser::HandleEscape(uint32 Codepoint)
{
	// C0 controls mid-escape are executed.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// Intermediate bytes (0x20-0x2F) -> collect, transition to EscapeIntermediate.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		EscapeIntermediateChars.Empty();
		EscapeIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		State = EParserState::EscapeIntermediate;
		return;
	}

	// DEL  - ignore.
	if (Codepoint == 0x7F)
	{
		return;
	}

	// Dispatch based on the byte after ESC.
	switch (Codepoint)
	{
	case '[': // CSI introducer (ESC [)
		ClearSequenceState();
		State = EParserState::CSIEntry;
		break;
	case ']': // OSC introducer (ESC ])
		OSCPayload.Empty();
		State = EParserState::OSCString;
		break;
	case 'P': // DCS introducer (ESC P)
		ClearSequenceState();
		State = EParserState::DCSEntry;
		break;
	case 'X': // SOS (ESC X)
	case '^': // PM (ESC ^)
	case '_': // APC (ESC _)
		State = EParserState::SOSPMAPCString;
		break;
	case '7': // DECSC  - save cursor
		Buffer->Cursor.SavedRow = Buffer->Cursor.Row;
		Buffer->Cursor.SavedColumn = Buffer->Cursor.Column;
		State = EParserState::Ground;
		break;
	case '8': // DECRC  - restore cursor
		Buffer->Cursor.Row = Buffer->Cursor.SavedRow;
		Buffer->Cursor.Column = Buffer->Cursor.SavedColumn;
		State = EParserState::Ground;
		break;
	case 'D': // IND  - index (move cursor down, scroll if at bottom)
		LineFeed();
		State = EParserState::Ground;
		break;
	case 'E': // NEL  - next line (CR + LF)
		Buffer->Cursor.Column = 0;
		LineFeed();
		State = EParserState::Ground;
		break;
	case 'M': // RI  - reverse index (move cursor up, scroll if at top)
	{
		const int32 Top = GetEffectiveScrollTop();
		if (Buffer->Cursor.Row == Top)
		{
			Buffer->ScrollRegionDown(Top, GetEffectiveScrollBottom(), 1);
		}
		else if (Buffer->Cursor.Row > 0)
		{
			Buffer->Cursor.Row--;
		}
		State = EParserState::Ground;
		break;
	}
	case 'c': // RIS  - full reset
		Reset();
		if (Buffer)
		{
			Buffer->Clear();
		}
		break;
	case 'H': // HTS  - set tab stop at current column
		if (Buffer->Cursor.Column < TabStops.Num())
		{
			TabStops[Buffer->Cursor.Column] = true;
		}
		State = EParserState::Ground;
		break;
	case '\\': // ST  - string terminator (terminates OSC/DCS/etc. via ESC \)
		// When ESC aborts an OSC via the "anywhere" transition, the payload is still
		// accumulated in OSCPayload. Dispatch it now that ST has arrived.
		if (!OSCPayload.IsEmpty())
		{
			ExecuteOSC();
			OSCPayload.Empty();
		}
		State = EParserState::Ground;
		break;
	default:
		// All other bytes 0x30-0x7E are dispatched as escape sequences.
		if (Codepoint >= 0x30 && Codepoint <= 0x7E)
		{
			ExecuteEscapeSequence(Codepoint);
		}
		State = EParserState::Ground;
		break;
	}
}

// -- EscapeIntermediate --

void FVTParser::HandleEscapeIntermediate(uint32 Codepoint)
{
	// C0 controls mid-escape are executed.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// More intermediate bytes (0x20-0x2F)  - collect.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (EscapeIntermediateChars.Len() < MaxIntermediateChars)
		{
			EscapeIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		return;
	}

	// DEL  - ignore.
	if (Codepoint == 0x7F)
	{
		return;
	}

	// Final byte (0x30-0x7E)  - dispatch the escape sequence with intermediates.
	if (Codepoint >= 0x30 && Codepoint <= 0x7E)
	{
		ExecuteEscapeSequence(Codepoint);
	}
	State = EParserState::Ground;
}

// -- CSI Entry --

void FVTParser::HandleCSIEntry(uint32 Codepoint)
{
	// C0 controls  - execute inline.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// Intermediate bytes (0x20-0x2F)  - collect, go to CSIIntermediate.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		State = EParserState::CSIIntermediate;
		return;
	}

	// Parameter digits and semicolons (0x30-0x39, 0x3B).
	if ((Codepoint >= 0x30 && Codepoint <= 0x39) || Codepoint == 0x3B)
	{
		State = EParserState::CSIParam;
		HandleCSIParam(Codepoint);
		return;
	}

	// Colon (0x3A)  - sub-parameter separator. Treat like semicolon for modern SGR.
	if (Codepoint == 0x3A)
	{
		State = EParserState::CSIParam;
		HandleCSIParam(Codepoint);
		return;
	}

	// Private parameter prefix bytes (0x3C-0x3F): '<', '=', '>', '?'.
	if (Codepoint >= 0x3C && Codepoint <= 0x3F)
	{
		CSIPrivatePrefix = static_cast<TCHAR>(Codepoint);
		State = EParserState::CSIParam;
		return;
	}

	// Final byte (0x40-0x7E)  - dispatch immediately (no params).
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		ExecuteCSI(Codepoint);
		State = EParserState::Ground;
		return;
	}

	// DEL  - ignore.
	if (Codepoint == 0x7F)
	{
		return;
	}
}

// -- CSI Param --

void FVTParser::HandleCSIParam(uint32 Codepoint)
{
	// C0 controls  - execute inline per spec.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// Parameter digits (0x30-0x39).
	if (Codepoint >= 0x30 && Codepoint <= 0x39)
	{
		if (CurrentParam.Len() < MaxCSIParamDigits)
		{
			CurrentParam.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		return;
	}

	// Semicolons (0x3B) and colons (0x3A)  - parameter/sub-parameter separators.
	// Colons are a modern extension (e.g., 38:2:R:G:B for SGR true-color).
	// We treat them identically to semicolons for parameter splitting.
	if (Codepoint == 0x3B || Codepoint == 0x3A)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		return;
	}

	// Private marker bytes (0x3C-0x3F) appearing AFTER the initial position is an error.
	// Transition to CSIIgnore to consume the rest of the malformed sequence.
	if (Codepoint >= 0x3C && Codepoint <= 0x3F)
	{
		State = EParserState::CSIIgnore;
		return;
	}

	// Intermediate bytes (0x20-0x2F)  - collect and transition to CSIIntermediate.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		State = EParserState::CSIIntermediate;
		return;
	}

	// Final byte (0x40-0x7E)  - flush last param, dispatch.
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		ExecuteCSI(Codepoint);
		State = EParserState::Ground;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- CSI Intermediate --

void FVTParser::HandleCSIIntermediate(uint32 Codepoint)
{
	// C0 controls  - execute inline.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// More intermediate bytes (0x20-0x2F)  - collect.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		return;
	}

	// Parameter bytes (0x30-0x3F) after intermediates = error -> CSIIgnore.
	if (Codepoint >= 0x30 && Codepoint <= 0x3F)
	{
		State = EParserState::CSIIgnore;
		return;
	}

	// Final byte (0x40-0x7E)  - dispatch.
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		ExecuteCSI(Codepoint);
		State = EParserState::Ground;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- CSI Ignore --

void FVTParser::HandleCSIIgnore(uint32 Codepoint)
{
	// C0 controls  - execute inline even in ignore state.
	if (Codepoint < 0x20)
	{
		TryExecuteC0(Codepoint);
		return;
	}

	// Consume everything (0x20-0x3F) without action.
	if (Codepoint >= 0x20 && Codepoint <= 0x3F)
	{
		return;
	}

	// Final byte (0x40-0x7E)  - go to ground WITHOUT dispatching.
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		State = EParserState::Ground;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- OSC String --

void FVTParser::HandleOSCString(uint32 Codepoint)
{
	// BEL (0x07) terminates OSC (xterm extension, universally supported).
	if (Codepoint == 0x07)
	{
		ExecuteOSC();
		State = EParserState::Ground;
		return;
	}

	// C0 controls other than BEL  - ignore per spec.
	if (Codepoint < 0x20)
	{
		return;
	}

	// DEL (0x7F)  - ignore per spec (some implementations allow it as data).
	if (Codepoint == 0x7F)
	{
		return;
	}

	// ESC and 8-bit ST (0x9C) are handled by "anywhere" transitions in
	// ProcessCodepoint. The ESC transition moves to Escape state; the subsequent
	// '\' dispatches the accumulated OSC payload from HandleEscape's ST case.

	// Collect printable bytes as OSC data, capped to prevent unbounded growth.
	if (OSCPayload.Len() < MaxOSCPayloadLength)
	{
		OSCPayload.AppendChar(static_cast<TCHAR>(Codepoint));
	}
}

// -- DCS Entry --

void FVTParser::HandleDCSEntry(uint32 Codepoint)
{
	// C0 controls  - IGNORED (not executed) in DCS states per spec.
	if (Codepoint < 0x20)
	{
		return;
	}

	// Intermediate bytes (0x20-0x2F)  - collect, go to DCSIntermediate.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		State = EParserState::DCSIntermediate;
		return;
	}

	// Parameter digits and semicolons (0x30-0x39, 0x3B).
	if ((Codepoint >= 0x30 && Codepoint <= 0x39) || Codepoint == 0x3B)
	{
		State = EParserState::DCSParam;
		HandleDCSParam(Codepoint);
		return;
	}

	// Colon (0x3A)  - error -> DCSIgnore.
	if (Codepoint == 0x3A)
	{
		State = EParserState::DCSIgnore;
		return;
	}

	// Private prefix bytes (0x3C-0x3F)  - collect, go to DCSParam.
	if (Codepoint >= 0x3C && Codepoint <= 0x3F)
	{
		CSIPrivatePrefix = static_cast<TCHAR>(Codepoint);
		State = EParserState::DCSParam;
		return;
	}

	// Final byte (0x40-0x7E)  - go directly to DCSPassthrough (hook).
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		State = EParserState::DCSPassthrough;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- DCS Param --

void FVTParser::HandleDCSParam(uint32 Codepoint)
{
	// C0 controls  - ignored in DCS.
	if (Codepoint < 0x20)
	{
		return;
	}

	// Parameter digits (0x30-0x39).
	if (Codepoint >= 0x30 && Codepoint <= 0x39)
	{
		if (CurrentParam.Len() < MaxCSIParamDigits)
		{
			CurrentParam.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		return;
	}

	// Semicolons (0x3B).
	if (Codepoint == 0x3B)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		return;
	}

	// Colon (0x3A)  - error -> DCSIgnore.
	if (Codepoint == 0x3A)
	{
		State = EParserState::DCSIgnore;
		return;
	}

	// Private marker bytes (0x3C-0x3F) after initial position = error -> DCSIgnore.
	if (Codepoint >= 0x3C && Codepoint <= 0x3F)
	{
		State = EParserState::DCSIgnore;
		return;
	}

	// Intermediate bytes (0x20-0x2F)  - collect, go to DCSIntermediate.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		State = EParserState::DCSIntermediate;
		return;
	}

	// Final byte (0x40-0x7E)  - flush, go to DCSPassthrough.
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		if (CSIParams.Num() < MaxCSIParamCount)
		{
			CSIParams.Add(CurrentParam.IsEmpty() ? 0 : FCString::Atoi(*CurrentParam));
		}
		CurrentParam.Empty();
		State = EParserState::DCSPassthrough;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- DCS Intermediate --

void FVTParser::HandleDCSIntermediate(uint32 Codepoint)
{
	// C0 controls  - ignored in DCS.
	if (Codepoint < 0x20)
	{
		return;
	}

	// More intermediate bytes (0x20-0x2F)  - collect.
	if (Codepoint >= 0x20 && Codepoint <= 0x2F)
	{
		if (CSIIntermediateChars.Len() < MaxIntermediateChars)
		{
			CSIIntermediateChars.AppendChar(static_cast<TCHAR>(Codepoint));
		}
		return;
	}

	// Parameter bytes (0x30-0x3F) after intermediates = error -> DCSIgnore.
	if (Codepoint >= 0x30 && Codepoint <= 0x3F)
	{
		State = EParserState::DCSIgnore;
		return;
	}

	// Final byte (0x40-0x7E)  - go to DCSPassthrough.
	if (Codepoint >= 0x40 && Codepoint <= 0x7E)
	{
		State = EParserState::DCSPassthrough;
		return;
	}

	// DEL (0x7F)  - ignore.
}

// -- DCS Passthrough --

void FVTParser::HandleDCSPassthrough(uint32 Codepoint)
{
	// C0 controls are passed through to the DCS handler (the "put" action).
	// Since we don't act on DCS data, we silently consume everything.
	// Termination is via "anywhere" transitions (ESC or 8-bit ST).

	// DEL (0x7F)  - ignore.
	// All other bytes (0x00-0x7E)  - silently consume.
}

// -- DCS Ignore --

void FVTParser::HandleDCSIgnore(uint32 Codepoint)
{
	// Silently consume all bytes until terminated by "anywhere" transition.
	// C0 controls are ignored. All other bytes (0x20-0x7F) are ignored.
}

// -- SOS/PM/APC String --

void FVTParser::HandleSOSPMAPCString(uint32 Codepoint)
{
	// Silently consume all data until terminated by "anywhere" transition
	// (ESC triggers Escape state, then '\' = ST; or 8-bit 0x9C = ST).
}

// -- Escape Sequence Dispatch (with intermediates) --

void FVTParser::ExecuteEscapeSequence(uint32 FinalByte)
{
	// This handles escape sequences that went through EscapeIntermediate,
	// or single-character escapes not caught by HandleEscape's switch.
	// Most escape-with-intermediate sequences are charset designations
	// (ESC ( B, ESC ) 0, etc.) which we silently ignore for now.
}

// -- CSI Dispatch --

void FVTParser::ExecuteCSI(uint32 FinalByte)
{
	if (CSIPrivatePrefix != 0)
	{
		ExecutePrivateMode(FinalByte);
		return;
	}

	switch (FinalByte)
	{
	case 'A': // CUU  - cursor up
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Row = FMath::Max(GetEffectiveScrollTop(), Buffer->Cursor.Row - Count);
		break;
	}
	case 'B': // CUD  - cursor down
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Row = FMath::Min(GetEffectiveScrollBottom(), Buffer->Cursor.Row + Count);
		break;
	}
	case 'C': // CUF  - cursor forward
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Column = FMath::Min(Buffer->GetColumns() - 1, Buffer->Cursor.Column + Count);
		break;
	}
	case 'D': // CUB  - cursor backward
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Column = FMath::Max(0, Buffer->Cursor.Column - Count);
		break;
	}
	case 'E': // CNL  - cursor next line
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Row = FMath::Min(GetEffectiveScrollBottom(), Buffer->Cursor.Row + Count);
		Buffer->Cursor.Column = 0;
		break;
	}
	case 'F': // CPL  - cursor previous line
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->Cursor.Row = FMath::Max(GetEffectiveScrollTop(), Buffer->Cursor.Row - Count);
		Buffer->Cursor.Column = 0;
		break;
	}
	case 'H': // CUP  - cursor position
	case 'f': // HVP  - same as CUP
	{
		const int32 Row = FMath::Max(1, GetCSIParam(0, 1)) - 1; // 1-based to 0-based
		const int32 Col = FMath::Max(1, GetCSIParam(1, 1)) - 1;
		Buffer->Cursor.Row = FMath::Clamp(Row, 0, Buffer->GetViewportRows() - 1);
		Buffer->Cursor.Column = FMath::Clamp(Col, 0, Buffer->GetColumns() - 1);
		break;
	}
	case 'J': // ED  - erase in display
	{
		const int32 Mode = GetCSIParam(0, 0);
		const int32 CursorAbsoluteRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);
		FTerminalCell BlankCell = Buffer->DefaultCell;
		BlankCell.Foreground = CurrentForeground;
		BlankCell.Background = CurrentBackground;

		if (Mode == 0) // Erase below (cursor to end)
		{
			// Current line from cursor to end.
			for (int32 Col = Buffer->Cursor.Column; Col < Buffer->GetColumns(); ++Col)
			{
				Buffer->SetCell(CursorAbsoluteRow, Col, BlankCell);
			}
			// Lines below cursor.
			for (int32 Row = Buffer->Cursor.Row + 1; Row < Buffer->GetViewportRows(); ++Row)
			{
				const int32 AbsRow = Buffer->GetAbsoluteRow(Row, 0);
				for (int32 Col = 0; Col < Buffer->GetColumns(); ++Col)
				{
					Buffer->SetCell(AbsRow, Col, BlankCell);
				}
			}
		}
		else if (Mode == 1) // Erase above (start to cursor)
		{
			for (int32 Row = 0; Row < Buffer->Cursor.Row; ++Row)
			{
				const int32 AbsRow = Buffer->GetAbsoluteRow(Row, 0);
				for (int32 Col = 0; Col < Buffer->GetColumns(); ++Col)
				{
					Buffer->SetCell(AbsRow, Col, BlankCell);
				}
			}
			for (int32 Col = 0; Col <= Buffer->Cursor.Column; ++Col)
			{
				Buffer->SetCell(CursorAbsoluteRow, Col, BlankCell);
			}
		}
		else if (Mode == 2) // Erase all  - push viewport into scrollback, then blank
		{
			for (int32 Row = 0; Row < Buffer->GetViewportRows(); ++Row)
			{
				Buffer->PushNewLine();
			}
		}
		else if (Mode == 3) // Erase saved lines  - clear scrollback
		{
			Buffer->ClearScrollback();
		}
		break;
	}
	case 'K': // EL  - erase in line
	{
		const int32 Mode = GetCSIParam(0, 0);
		const int32 CursorAbsoluteRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);
		FTerminalCell BlankCell = Buffer->DefaultCell;
		BlankCell.Foreground = CurrentForeground;
		BlankCell.Background = CurrentBackground;

		if (Mode == 0) // Erase to right
		{
			for (int32 Col = Buffer->Cursor.Column; Col < Buffer->GetColumns(); ++Col)
			{
				Buffer->SetCell(CursorAbsoluteRow, Col, BlankCell);
			}
		}
		else if (Mode == 1) // Erase to left
		{
			for (int32 Col = 0; Col <= Buffer->Cursor.Column; ++Col)
			{
				Buffer->SetCell(CursorAbsoluteRow, Col, BlankCell);
			}
		}
		else if (Mode == 2) // Erase entire line
		{
			for (int32 Col = 0; Col < Buffer->GetColumns(); ++Col)
			{
				Buffer->SetCell(CursorAbsoluteRow, Col, BlankCell);
			}
		}
		break;
	}
	case 'S': // SU  - scroll up
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->ScrollRegionUp(GetEffectiveScrollTop(), GetEffectiveScrollBottom(), Count);
		break;
	}
	case 'T': // SD  - scroll down
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->ScrollRegionDown(GetEffectiveScrollTop(), GetEffectiveScrollBottom(), Count);
		break;
	}
	case 'r': // DECSTBM  - set scroll region
	{
		const int32 Top = FMath::Max(1, GetCSIParam(0, 1)) - 1;
		const int32 Bottom = FMath::Max(1, GetCSIParam(1, Buffer->GetViewportRows())) - 1;
		if (Top < Bottom && Bottom < Buffer->GetViewportRows())
		{
			ScrollRegionTop = Top;
			ScrollRegionBottom = Bottom;
		}
		else
		{
			ScrollRegionTop = -1;
			ScrollRegionBottom = -1;
		}
		// DECSTBM also moves cursor to home.
		Buffer->Cursor.Row = 0;
		Buffer->Cursor.Column = 0;
		break;
	}
	case 'm': // SGR  - select graphic rendition
		ExecuteSGR();
		break;
	case 'G': // CHA  - cursor horizontal absolute
	case '`': // HPA  - same
	{
		const int32 Col = FMath::Max(1, GetCSIParam(0, 1)) - 1;
		Buffer->Cursor.Column = FMath::Clamp(Col, 0, Buffer->GetColumns() - 1);
		break;
	}
	case 'd': // VPA  - vertical position absolute
	{
		const int32 Row = FMath::Max(1, GetCSIParam(0, 1)) - 1;
		Buffer->Cursor.Row = FMath::Clamp(Row, 0, Buffer->GetViewportRows() - 1);
		break;
	}
	case 'L': // IL  - insert lines
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->ScrollRegionDown(Buffer->Cursor.Row, GetEffectiveScrollBottom(), Count);
		break;
	}
	case 'M': // DL  - delete lines
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		Buffer->ScrollRegionUp(Buffer->Cursor.Row, GetEffectiveScrollBottom(), Count);
		break;
	}
	case 'P': // DCH  - delete characters
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		const int32 AbsRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);
		const int32 Columns = Buffer->GetColumns();
		for (int32 Col = Buffer->Cursor.Column; Col < Columns; ++Col)
		{
			const int32 SourceCol = Col + Count;
			if (SourceCol < Columns)
			{
				Buffer->SetCell(AbsRow, Col, Buffer->GetCell(AbsRow, SourceCol));
			}
			else
			{
				Buffer->SetCell(AbsRow, Col, Buffer->DefaultCell);
			}
		}
		break;
	}
	case '@': // ICH  - insert characters
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		const int32 AbsRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);
		const int32 Columns = Buffer->GetColumns();
		for (int32 Col = Columns - 1; Col >= Buffer->Cursor.Column; --Col)
		{
			const int32 SourceCol = Col - Count;
			if (SourceCol >= Buffer->Cursor.Column)
			{
				Buffer->SetCell(AbsRow, Col, Buffer->GetCell(AbsRow, SourceCol));
			}
			else
			{
				Buffer->SetCell(AbsRow, Col, Buffer->DefaultCell);
			}
		}
		break;
	}
	case 'X': // ECH  - erase characters
	{
		const int32 Count = FMath::Max(1, GetCSIParam(0, 1));
		const int32 AbsRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);
		FTerminalCell BlankCell = Buffer->DefaultCell;
		BlankCell.Foreground = CurrentForeground;
		BlankCell.Background = CurrentBackground;
		for (int32 Index = 0; Index < Count && (Buffer->Cursor.Column + Index) < Buffer->GetColumns(); ++Index)
		{
			Buffer->SetCell(AbsRow, Buffer->Cursor.Column + Index, BlankCell);
		}
		break;
	}
	case 's': // SCOSC  - save cursor position
		Buffer->Cursor.SavedRow = Buffer->Cursor.Row;
		Buffer->Cursor.SavedColumn = Buffer->Cursor.Column;
		break;
	case 'u': // SCORC  - restore cursor position
		Buffer->Cursor.Row = Buffer->Cursor.SavedRow;
		Buffer->Cursor.Column = Buffer->Cursor.SavedColumn;
		break;
	case 'g': // TBC  - tab clear
	{
		const int32 Mode = GetCSIParam(0, 0);
		if (Mode == 0)
		{
			if (Buffer->Cursor.Column < TabStops.Num())
			{
				TabStops[Buffer->Cursor.Column] = false;
			}
		}
		else if (Mode == 3)
		{
			for (int32 Index = 0; Index < TabStops.Num(); ++Index)
			{
				TabStops[Index] = false;
			}
		}
		break;
	}
	case 'n': // DSR  - device status report
	{
		const int32 Mode = GetCSIParam(0, 0);
		if (Mode == 5)
		{
			// Operating status report: respond "OK".
			static const uint8 StatusOKResponse[] = { 0x1B, '[', '0', 'n' };
			ResponseBuffer.Append(StatusOKResponse, UE_ARRAY_COUNT(StatusOKResponse));
		}
		else if (Mode == 6 && Buffer)
		{
			// Cursor position report: ESC [ row ; col R  (1-based).
			const FString Report = FString::Printf(TEXT("\x1B[%d;%dR"), Buffer->Cursor.Row + 1, Buffer->Cursor.Column + 1);
			FTCHARToUTF8 ReportUtf8(*Report);
			ResponseBuffer.Append(reinterpret_cast<const uint8*>(ReportUtf8.Get()), ReportUtf8.Length());
		}
		break;
	}
	case 'c': // DA  - device attributes
	{
		// Respond as VT101 with no options.
		static const uint8 DeviceAttributesResponse[] = { 0x1B, '[', '?', '1', ';', '0', 'c' };
		ResponseBuffer.Append(DeviceAttributesResponse, UE_ARRAY_COUNT(DeviceAttributesResponse));
		break;
	}
	case 't': // XTWINOPS  - window manipulation (silently ignore)
		break;
	case 'b': // REP  - repeat previous character (silently ignore for now)
		break;
	default:
		// Unrecognized CSI  - silently ignore.
		break;
	}
}

// -- SGR (Select Graphic Rendition) --

void FVTParser::ExecuteSGR()
{
	if (CSIParams.Num() == 0)
	{
		CSIParams.Add(0);
	}

	for (int32 Index = 0; Index < CSIParams.Num(); ++Index)
	{
		const int32 Param = CSIParams[Index];

		switch (Param)
		{
		case 0: // Reset
			CurrentForeground = Buffer ? Buffer->DefaultCell.Foreground : FColor(212, 212, 212);
			CurrentBackground = Buffer ? Buffer->DefaultCell.Background : FColor(30, 30, 30);
			CurrentAttributes = ETerminalAttribute::None;
			break;
		case 1: CurrentAttributes |= ETerminalAttribute::Bold; break;
		case 2: CurrentAttributes |= ETerminalAttribute::Dim; break;
		case 3: CurrentAttributes |= ETerminalAttribute::Italic; break;
		case 4: CurrentAttributes |= ETerminalAttribute::Underline; break;
		case 7: CurrentAttributes |= ETerminalAttribute::Inverse; break;
		case 9: CurrentAttributes |= ETerminalAttribute::Strikethrough; break;
		case 22: CurrentAttributes &= ~(ETerminalAttribute::Bold | ETerminalAttribute::Dim); break;
		case 23: CurrentAttributes &= ~ETerminalAttribute::Italic; break;
		case 24: CurrentAttributes &= ~ETerminalAttribute::Underline; break;
		case 27: CurrentAttributes &= ~ETerminalAttribute::Inverse; break;
		case 29: CurrentAttributes &= ~ETerminalAttribute::Strikethrough; break;

		// Standard foreground colors (30-37).
		case 30: case 31: case 32: case 33:
		case 34: case 35: case 36: case 37:
			CurrentForeground = AnsiPalette[Param - 30];
			break;

		// Extended foreground (38;5;N or 38;2;R;G;B).
		case 38:
			if (Index + 1 < CSIParams.Num())
			{
				if (CSIParams[Index + 1] == 5 && Index + 2 < CSIParams.Num())
				{
					// 256-color
					const int32 ColorIndex = CSIParams[Index + 2];
					CurrentForeground = Get256Color(ColorIndex);
					Index += 2;
				}
				else if (CSIParams[Index + 1] == 2 && Index + 4 < CSIParams.Num())
				{
					// True-color
					CurrentForeground = FColor(
						FMath::Clamp(CSIParams[Index + 2], 0, 255),
						FMath::Clamp(CSIParams[Index + 3], 0, 255),
						FMath::Clamp(CSIParams[Index + 4], 0, 255));
					Index += 4;
				}
			}
			break;

		case 39: // Default foreground
			CurrentForeground = Buffer ? Buffer->DefaultCell.Foreground : FColor(212, 212, 212);
			break;

		// Standard background colors (40-47).
		case 40: case 41: case 42: case 43:
		case 44: case 45: case 46: case 47:
			CurrentBackground = AnsiPalette[Param - 40];
			break;

		// Extended background (48;5;N or 48;2;R;G;B).
		case 48:
			if (Index + 1 < CSIParams.Num())
			{
				if (CSIParams[Index + 1] == 5 && Index + 2 < CSIParams.Num())
				{
					const int32 ColorIndex = CSIParams[Index + 2];
					CurrentBackground = Get256Color(ColorIndex);
					Index += 2;
				}
				else if (CSIParams[Index + 1] == 2 && Index + 4 < CSIParams.Num())
				{
					CurrentBackground = FColor(
						FMath::Clamp(CSIParams[Index + 2], 0, 255),
						FMath::Clamp(CSIParams[Index + 3], 0, 255),
						FMath::Clamp(CSIParams[Index + 4], 0, 255));
					Index += 4;
				}
			}
			break;

		case 49: // Default background
			CurrentBackground = Buffer ? Buffer->DefaultCell.Background : FColor(30, 30, 30);
			break;

		// Bright foreground colors (90-97).
		case 90: case 91: case 92: case 93:
		case 94: case 95: case 96: case 97:
			CurrentForeground = AnsiPalette[Param - 90 + 8];
			break;

		// Bright background colors (100-107).
		case 100: case 101: case 102: case 103:
		case 104: case 105: case 106: case 107:
			CurrentBackground = AnsiPalette[Param - 100 + 8];
			break;

		default:
			break;
		}
	}
}

// -- Private Mode (CSI ? / CSI > / etc.) --

void FVTParser::ExecutePrivateMode(uint32 FinalByte)
{
	// Only handle '?' prefix for DECSET/DECRST. Other prefixes ('<', '=', '>')
	// are silently ignored (e.g., Kitty keyboard protocol pushes via CSI > u).
	if (CSIPrivatePrefix != '?')
	{
		return;
	}

	for (int32 Index = 0; Index < CSIParams.Num(); ++Index)
	{
		const int32 Mode = CSIParams[Index];

		switch (FinalByte)
		{
		case 'h': // DECSET  - set mode
			switch (Mode)
			{
			case 1: bApplicationCursorKeys = true; break;
			case 7: bAutoWrap = true; break;
			case 25: bCursorVisible = true; break;
			case 1000: MouseTrackingMode = EMouseTrackingMode::Normal; break;
			case 1002: MouseTrackingMode = EMouseTrackingMode::ButtonEvent; break;
			case 1003: MouseTrackingMode = EMouseTrackingMode::Any; break;
			case 1006: bSGRMouseEncoding = true; break;
			case 1049: // Alternate screen buffer
				Buffer->ActivateAlternateBuffer();
				break;
			case 2004: bBracketedPaste = true; break;
			case 2026: bSynchronizedOutput = true; break;
			}
			break;
		case 'l': // DECRST  - reset mode
			switch (Mode)
			{
			case 1: bApplicationCursorKeys = false; break;
			case 7: bAutoWrap = false; break;
			case 25: bCursorVisible = false; break;
			case 1000: // Falls through - any mouse tracking reset disables tracking.
			case 1002:
			case 1003: MouseTrackingMode = EMouseTrackingMode::None; break;
			case 1006: bSGRMouseEncoding = false; break;
			case 1049:
				Buffer->DeactivateAlternateBuffer();
				break;
			case 2004: bBracketedPaste = false; break;
			case 2026: bSynchronizedOutput = false; break;
			}
			break;
		}
	}
}

// -- OSC Dispatch --

void FVTParser::ExecuteOSC()
{
	// Parse OSC payload: "Ps;Pt" where Ps is the command number.
	int32 SemicolonIndex;
	if (OSCPayload.FindChar(TEXT(';'), SemicolonIndex))
	{
		const FString CommandStr = OSCPayload.Left(SemicolonIndex);
		const int32 Command = FCString::Atoi(*CommandStr);
		const FString Payload = OSCPayload.Mid(SemicolonIndex + 1);

		switch (Command)
		{
		case 0: // Set window title and icon name
		case 2: // Set window title
			WindowTitle = Payload;
			break;
		default:
			// Ignore other OSC commands.
			break;
		}
	}
}

// -- Character Output --

bool FVTParser::IsZeroWidthCodepoint(uint32 Codepoint)
{
	// Variation Selectors (emoji presentation, text presentation, etc.)
	if (Codepoint >= 0xFE00 && Codepoint <= 0xFE0F) return true;
	// Combining Diacritical Marks
	if (Codepoint >= 0x0300 && Codepoint <= 0x036F) return true;
	// Zero-width space, non-joiner, joiner
	if (Codepoint >= 0x200B && Codepoint <= 0x200D) return true;
	// Zero-width no-break space (BOM)
	if (Codepoint == 0xFEFF) return true;
	// Combining Diacritical Marks Extended
	if (Codepoint >= 0x1AB0 && Codepoint <= 0x1AFF) return true;
	// Combining Diacritical Marks Supplement
	if (Codepoint >= 0x1DC0 && Codepoint <= 0x1DFF) return true;
	// Combining Diacritical Marks for Symbols
	if (Codepoint >= 0x20D0 && Codepoint <= 0x20FF) return true;
	// Combining Half Marks
	if (Codepoint >= 0xFE20 && Codepoint <= 0xFE2F) return true;
	// Variation Selectors Supplement
	if (Codepoint >= 0xE0100 && Codepoint <= 0xE01EF) return true;
	return false;
}

void FVTParser::PutCharacter(TCHAR Character)
{
	if (!Buffer)
	{
		return;
	}

	const int32 Columns = Buffer->GetColumns();

	// Auto-wrap: if cursor is past the last column, wrap to next line.
	if (Buffer->Cursor.Column >= Columns)
	{
		if (bAutoWrap)
		{
			Buffer->Cursor.Column = 0;
			LineFeed();
		}
		else
		{
			Buffer->Cursor.Column = Columns - 1;
		}
	}

	const int32 AbsoluteRow = Buffer->GetAbsoluteRow(Buffer->Cursor.Row, 0);

	FTerminalCell Cell;
	Cell.Character = Character;
	Cell.Foreground = CurrentForeground;
	Cell.Background = CurrentBackground;
	Cell.Attributes = CurrentAttributes;

	Buffer->SetCell(AbsoluteRow, Buffer->Cursor.Column, Cell);
	Buffer->Cursor.Column++;
}

void FVTParser::LineFeed()
{
	if (!Buffer)
	{
		return;
	}

	const int32 Bottom = GetEffectiveScrollBottom();

	if (Buffer->Cursor.Row == Bottom)
	{
		// At the bottom of the scroll region  - scroll up.
		const int32 Top = GetEffectiveScrollTop();
		if (Top == 0 && Bottom == Buffer->GetViewportRows() - 1)
		{
			// Full-screen scroll  - push a new line into the ring buffer (preserves scrollback).
			Buffer->PushNewLine();
		}
		else
		{
			// Scroll within a region.
			Buffer->ScrollRegionUp(Top, Bottom, 1);
		}
	}
	else if (Buffer->Cursor.Row < Buffer->GetViewportRows() - 1)
	{
		Buffer->Cursor.Row++;
	}
}

int32 FVTParser::GetEffectiveScrollTop() const
{
	return (ScrollRegionTop >= 0) ? ScrollRegionTop : 0;
}

int32 FVTParser::GetEffectiveScrollBottom() const
{
	if (!Buffer)
	{
		return 0;
	}
	return (ScrollRegionBottom >= 0) ? ScrollRegionBottom : (Buffer->GetViewportRows() - 1);
}

void FVTParser::InitializeTabStops()
{
	const int32 MaxColumns = Buffer ? Buffer->GetColumns() : 256;
	TabStops.SetNum(MaxColumns);
	for (int32 Index = 0; Index < MaxColumns; ++Index)
	{
		TabStops[Index] = (Index % 8 == 0);
	}
}

int32 FVTParser::GetCSIParam(int32 Index, int32 Default) const
{
	if (Index < CSIParams.Num() && CSIParams[Index] != 0)
	{
		return CSIParams[Index];
	}
	return Default;
}

FColor FVTParser::Get256Color(int32 Index) const
{
	// Standard 16 colors (0-15)  - use the configurable ANSI palette.
	if (Index >= 0 && Index < 16)
	{
		return AnsiPalette[Index];
	}

	// 216-color cube (16-231).
	if (Index >= 16 && Index < 232)
	{
		const int32 CubeIndex = Index - 16;
		const int32 Red = CubeIndex / 36;
		const int32 Green = (CubeIndex / 6) % 6;
		const int32 Blue = CubeIndex % 6;
		return FColor(
			Red ? (Red * 40 + 55) : 0,
			Green ? (Green * 40 + 55) : 0,
			Blue ? (Blue * 40 + 55) : 0);
	}

	// Grayscale ramp (232-255).
	if (Index >= 232 && Index < 256)
	{
		const int32 Gray = (Index - 232) * 10 + 8;
		return FColor(Gray, Gray, Gray);
	}

	return FColor(255, 255, 255);
}
