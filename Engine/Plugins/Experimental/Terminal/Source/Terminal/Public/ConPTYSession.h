// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITerminalSession.h"

#if PLATFORM_WINDOWS

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/CriticalSection.h"

/**
 * Manages a ConPTY pseudo-console session on Windows.
 *
 * Creates a pseudo-console via the ConPTY API (dynamically loaded),
 * spawns a shell process attached to it, and runs asynchronous I/O
 * on a background thread with staged dispatch to the game thread.
 */
class FConPTYSession : public ITerminalSession
{
public:

	FConPTYSession();
	virtual ~FConPTYSession() override;

	FConPTYSession(const FConPTYSession&) = delete;
	FConPTYSession& operator=(const FConPTYSession&) = delete;

	/** Check if the ConPTY API is available on this system. */
	static bool IsConPTYAvailable();

	virtual bool Create(const FString& ShellPath, const FString& WorkingDirectory, int32 Columns, int32 Rows) override;
	virtual void Shutdown() override;
	virtual void WriteInput(TArrayView<const uint8> Data) override;
	virtual void Resize(int32 Columns, int32 Rows) override;
	virtual TArray<uint8> ConsumeOutput() override;
	virtual bool IsRunning() const override;

private:

	/** Background I/O reader thread. */
	class FReaderRunnable : public FRunnable
	{
	public:
		FReaderRunnable(FConPTYSession* InOwner) : Owner(InOwner) {}
		virtual uint32 Run() override;
		virtual void Stop() override;
	private:
		FConPTYSession* Owner = nullptr;
		TAtomic<bool> bStopRequested{false};
	};

	void* PseudoConsoleHandle = nullptr;
	void* InputPipeWrite = nullptr;
	void* OutputPipeRead = nullptr;
	void* ProcessHandle = nullptr;
	void* ThreadHandle = nullptr;

	/** Shared alive flag for safe async callback dispatch after destruction. */
	TSharedPtr<TAtomic<bool>> bAlive;

	/** Output staging buffer  - filled by the reader thread, drained by ConsumeOutput() on the game thread. */
	TArray<uint8> StagingBuffer;
	FCriticalSection StagingLock;

	TUniquePtr<FReaderRunnable> ReaderRunnable;
	TUniquePtr<FRunnableThread> ReaderThread;
};

#endif // PLATFORM_WINDOWS
