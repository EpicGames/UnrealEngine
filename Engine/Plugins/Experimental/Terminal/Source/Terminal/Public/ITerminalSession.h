// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTerminal, Log, All);

DECLARE_DELEGATE_OneParam(FOnTerminalProcessExited, int32 /* ExitCode */);

/**
 * Abstract interface for a terminal PTY session.
 *
 * Platform-specific backends (FConPTYSession on Windows, FPosixPTYSession on
 * Linux/macOS) implement this interface so that the terminal widget can drive
 * any backend without platform conditionals in its core logic.
 */
class ITerminalSession
{
public:

	virtual ~ITerminalSession() = default;

	/**
	 * Create a platform-appropriate session backend.
	 *
	 * Returns nullptr with an error message if no backend is available (e.g.
	 * ConPTY missing on older Windows, or unsupported platform).
	 *
	 * @param OutError Receives a user-visible error string when nullptr is returned.
	 */
	static TSharedPtr<ITerminalSession> CreateForCurrentPlatform(FString& OutError);

	/**
	 * Create a new session.
	 *
	 * @param ShellPath Path to the shell executable. Empty uses the system default.
	 * @param WorkingDirectory Initial working directory for the shell.
	 * @param Columns Initial column count.
	 * @param Rows Initial row count.
	 * @return true if the session was created successfully.
	 */
	virtual bool Create(const FString& ShellPath, const FString& WorkingDirectory, int32 Columns, int32 Rows) = 0;

	/** Shut down the session, closing the PTY and terminating the shell process. */
	virtual void Shutdown() = 0;

	/** Write input bytes to the PTY input pipe. */
	virtual void WriteInput(TArrayView<const uint8> Data) = 0;

	/** Resize the PTY. */
	virtual void Resize(int32 Columns, int32 Rows) = 0;

	/** Swap the output staging buffer and return the accumulated bytes for parsing on the game thread. */
	virtual TArray<uint8> ConsumeOutput() = 0;

	/** Check if the shell process is still running. */
	virtual bool IsRunning() const = 0;

	/** Delegate fired when the shell process exits. */
	FOnTerminalProcessExited OnProcessExited;
};
