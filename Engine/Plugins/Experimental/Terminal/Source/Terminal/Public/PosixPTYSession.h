// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITerminalSession.h"

#if PLATFORM_UNIX || PLATFORM_MAC

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/CriticalSection.h"

/**
 * Manages a POSIX PTY session on Linux and macOS.
 *
 * Creates a pseudo-terminal via forkpty(), spawns a shell process, and
 * runs asynchronous I/O on a background thread with staged dispatch to
 * the game thread  - mirroring the FConPTYSession pattern on Windows.
 */
class FPosixPTYSession : public ITerminalSession
{
public:

	FPosixPTYSession();
	virtual ~FPosixPTYSession() override;

	FPosixPTYSession(const FPosixPTYSession&) = delete;
	FPosixPTYSession& operator=(const FPosixPTYSession&) = delete;

	virtual bool Create(const FString& ShellPath, const FString& WorkingDirectory, int32 Columns, int32 Rows) override;
	virtual void Shutdown() override;
	virtual void WriteInput(TArrayView<const uint8> Data) override;
	virtual void Resize(int32 Columns, int32 Rows) override;
	virtual TArray<uint8> ConsumeOutput() override;
	virtual bool IsRunning() const override;

private:

	/** Background I/O reader thread. */
	class FReaderRunnable : public FRunnable
	{
	public:
		FReaderRunnable(FPosixPTYSession* InOwner) : Owner(InOwner) {}
		virtual uint32 Run() override;
		virtual void Stop() override;
	private:
		FPosixPTYSession* Owner = nullptr;
		TAtomic<bool> bStopRequested{false};
	};

	/** Master file descriptor of the pseudo-terminal. */
	int MasterFileDescriptor = -1;

	/** PID of the child shell process. */
	pid_t ProcessId = -1;

	/** Shared alive flag for safe async callback dispatch after destruction. */
	TSharedPtr<TAtomic<bool>> bAlive;

	/** Output staging buffer  - filled by the reader thread, drained by ConsumeOutput() on the game thread. */
	TArray<uint8> StagingBuffer;
	FCriticalSection StagingLock;

	TUniquePtr<FReaderRunnable> ReaderRunnable;
	TUniquePtr<FRunnableThread> ReaderThread;
};

#endif // PLATFORM_UNIX || PLATFORM_MAC
