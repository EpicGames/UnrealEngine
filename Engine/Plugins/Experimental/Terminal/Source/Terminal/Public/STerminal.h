// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/CompositeFont.h"
#include "Widgets/SLeafWidget.h"
#include "TerminalBuffer.h"
#include "VTParser.h"

class ITerminalSession;
class SScrollBar;
struct FTerminalColorScheme;

/** Fired on the game thread after the terminal consumes a non-empty chunk of session output. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTerminalOutputReceived, int32 /* NumBytes */);

/**
 * Native Slate terminal emulator widget.
 *
 * Renders a character cell grid directly via OnPaint using FSlateDrawElement.
 * Backed by FTerminalBuffer + FVTParser + FConPTYSession.
 */
class STerminal : public SLeafWidget
{
	friend class FSTerminalSpec;

public:

	SLATE_BEGIN_ARGS(STerminal)
	{}
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalScrollbar)
	SLATE_END_ARGS()

	TERMINAL_API void Construct(const FArguments& InArgs);

	TERMINAL_API virtual ~STerminal() override;

	/** Write a command string to the terminal session followed by a carriage return. */
	TERMINAL_API void ExecuteCommand(const FString& Command);

	/** Returns true once the terminal session has been created and is running. */
	TERMINAL_API bool IsSessionRunning() const;

	/** Broadcast on the game thread after each tick that consumed a non-empty output chunk from the session. */
	FOnTerminalOutputReceived OnOutputReceived;

	/** Engine time of the last non-empty output chunk, or 0.0 if none. */
	double GetLastOutputTime() const { return LastOutputTime; }

	/** Snapshot of every live STerminal (and subclass). Game thread only. Expired entries are pruned. */
	TERMINAL_API static TArray<TSharedRef<STerminal>> GetAllInstances();

	//~ Begin SWidget interface
	TERMINAL_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	TERMINAL_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	TERMINAL_API virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) override;
	TERMINAL_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	TERMINAL_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TERMINAL_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TERMINAL_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TERMINAL_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TERMINAL_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	TERMINAL_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	TERMINAL_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	TERMINAL_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	//~ End SWidget interface

private:

	/** Tick handler  - consumes session output and parses it. */
	EActiveTimerReturnType OnTick(double InCurrentTime, float InDeltaTime);

	/** Cursor blink timer. */
	EActiveTimerReturnType OnCursorBlink(double InCurrentTime, float InDeltaTime);

	/** Measure the monospace cell dimensions from the current font at the given layout scale. */
	void MeasureCellSize(float LayoutScale = 1.0f);

	/** Deferred initialization  - creates buffer and terminal session at correct dimensions. */
	void InitializeTerminal();

	/** Recompute viewport rows and columns from the widget's allocated geometry. */
	void UpdateViewportDimensions(const FGeometry& AllottedGeometry);

	/** Update the external scrollbar thumb position and size to reflect current scroll state. */
	void UpdateScrollBar();

	/** Convert a local pixel position to a buffer-absolute cell coordinate. */
	TERMINAL_API FIntPoint PixelToCell(const FGeometry& MyGeometry, const FVector2D& LocalPosition) const;

	/** Check if a cell is within the current selection range. */
	TERMINAL_API bool IsCellSelected(int32 AbsoluteRow, int32 Column) const;

	/** Get the selected text as a string. */
	TERMINAL_API FString GetSelectedText() const;

	/** Clear the current selection. */
	TERMINAL_API void ClearSelection();

	/** Paint per-cell backgrounds (inverse-aware). */
	void PaintBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	/** Paint text with monospace-safe batching, underline, and strikethrough. */
	void PaintText(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	/** Paint foreground overlays (selection highlight, cursor). */
	void PaintForeground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	/** Compute the effective foreground color for a cell, applying inverse, bold, and dim. */
	static FColor ComputeForegroundColor(const FTerminalCell& Cell);

	/** Apply a color scheme to default cell colors. */
	void ApplyColorScheme(const FTerminalColorScheme& Scheme);

	/** Called when the shell process exits. */
	void HandleProcessExited(int32 ExitCode);

	/** Called when the user drags the external scrollbar. */
	void HandleScrollBarScrolled(float InScrollOffsetFraction);

	/** Send a mouse event escape sequence to the session when mouse tracking is active. */
	void SendMouseEvent(int32 Button, int32 Column, int32 Row, bool bRelease);

	/** Shift+Arrow: start or extend the keyboard selection. Returns a handled reply when consumed. */
	TERMINAL_API TOptional<FReply> HandleShiftArrowSelection(const FKeyEvent& InKeyEvent);

	/** Copy the current selection to the OS clipboard on the platform's copy shortcut.
	 *  Returns a handled reply when the selection was copied; empty otherwise so the caller can fall
	 *  through (e.g. sending ETX on Ctrl+C without a selection). */
	TERMINAL_API TOptional<FReply> HandleCopyShortcut(const FKeyEvent& InKeyEvent, bool bCopyPasteModifier);

	/** Paste the OS clipboard into the session on the platform's paste shortcut, wrapping in bracketed
	 *  paste sequences when the parser is in bracketed-paste mode. Returns a handled reply when the
	 *  event matched the paste shortcut; empty otherwise. */
	TERMINAL_API TOptional<FReply> HandlePasteShortcut(const FKeyEvent& InKeyEvent, bool bCopyPasteModifier);

	/** Core terminal components. */
	FTerminalBuffer Buffer;
	FVTParser Parser;
	TSharedPtr<ITerminalSession> Session;
	TSharedPtr<SScrollBar> ScrollBar;

	/** Font and rendering. */
	TSharedPtr<FStandaloneCompositeFont> TerminalFont;
	FSlateFontInfo FontInfo;
	float CellWidth = 8.0f;
	float CellHeight = 16.0f;
	FColor SelectionHighlightColor = FColor(38, 79, 120);
	FColor CursorDisplayColor = FColor(174, 175, 173, 180);

	/** Viewport dimensions. */
	int32 ViewportColumns = 80;
	int32 ViewportRows = 24;
	int32 ScrollOffset = 0;
	mutable FVector2D LastAllocatedSize = FVector2D::ZeroVector;

	/** Per-tick parse budget in bytes. */
	static constexpr int32 ParseBudgetPerTick = 256 * 1024;

	/** Leftover bytes from a previous tick that exceeded the parse budget. */
	TArray<uint8> PendingParseData;

	/** Selection state (buffer-absolute coordinates). */
	FIntPoint SelectionAnchor = FIntPoint(-1, -1);
	FIntPoint SelectionActive = FIntPoint(-1, -1);

	/** Auto-scroll during drag selection: positive = scroll up (older), negative = scroll down (newer), 0 = none. */
	int32 SelectionAutoScrollDirection = 0;
	float SelectionAutoScrollAccumulator = 0.0f;

	/** Click tracking for double/triple click. */
	int32 ClickCount = 0;
	double LastClickTime = 0.0;
	FIntPoint LastClickCell = FIntPoint(-1, -1);

	/** Diagnostic counters. */
	int64 TotalBytesConsumed = 0;
	double LastOutputTime = 0.0;
	double LastHeartbeatLogTime = 0.0;

	/** State flags. */
	bool bInitialized = false;
	bool bSessionDead = false;
	bool bReceivedFirstOutput = false;
	bool bHasFocus = false;
	bool bSelecting = false;
	bool bHasSelection = false;
};
