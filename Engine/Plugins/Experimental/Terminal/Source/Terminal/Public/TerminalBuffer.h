// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Bitfield for terminal cell style attributes. */
namespace ETerminalAttribute
{
	enum Type : uint8
	{
		None          = 0,
		Bold          = 1 << 0,
		Dim           = 1 << 1,
		Italic        = 1 << 2,
		Underline     = 1 << 3,
		Inverse       = 1 << 4,
		Strikethrough = 1 << 5,
	};
}

/** A single character cell in the terminal grid. */
struct FTerminalCell
{
	TCHAR Character = TEXT(' ');
	FColor Foreground = FColor(212, 212, 212);
	FColor Background = FColor(30, 30, 30);
	uint8 Attributes = ETerminalAttribute::None;
};

/** Cursor state for the terminal. */
struct FTerminalCursor
{
	int32 Row = 0;
	int32 Column = 0;
	bool bBlinkOn = true;

	/** Saved cursor position for DECSC/DECRC. */
	int32 SavedRow = 0;
	int32 SavedColumn = 0;
};

/**
 * Ring-buffer-backed character cell grid with configurable scrollback.
 *
 * Rows are stored in a flat ring buffer. The viewport (the visible portion)
 * is always the last ViewportRows rows of the active buffer. Scrollback rows
 * accumulate above the viewport up to ScrollbackLimit.
 */
class FTerminalBuffer
{
public:

	TERMINAL_API FTerminalBuffer();

	TERMINAL_API void Initialize(int32 InColumns, int32 InViewportRows, int32 InScrollbackLimit);

	TERMINAL_API void Resize(int32 InColumns, int32 InViewportRows);

	/** Get a cell by absolute row index (0 = oldest scrollback row) and column. */
	TERMINAL_API const FTerminalCell& GetCell(int32 AbsoluteRow, int32 Column) const;

	/** Set a cell by absolute row index and column. */
	TERMINAL_API void SetCell(int32 AbsoluteRow, int32 Column, const FTerminalCell& Cell);

	/** Push a new blank line at the bottom, advancing the ring buffer. */
	TERMINAL_API void PushNewLine();

	/** Get the absolute row index for a given viewport row (0 = top of viewport) and scroll offset. */
	TERMINAL_API int32 GetAbsoluteRow(int32 ViewportRow, int32 ScrollOffset) const;

	/** Clear the entire buffer. */
	TERMINAL_API void Clear();

	/** Get the number of rows available for scrollback (above the viewport). */
	TERMINAL_API int32 GetScrollbackLength() const;

	/** Get the total number of rows written (viewport + scrollback). */
	TERMINAL_API int32 GetTotalRows() const;

	int32 GetColumns() const { return Columns; }
	int32 GetViewportRows() const { return ViewportRows; }

	/** Scroll a region of viewport rows up by Count lines, filling new lines with blank cells. */
	TERMINAL_API void ScrollRegionUp(int32 TopRow, int32 BottomRow, int32 Count);

	/** Scroll a region of viewport rows down by Count lines, filling new lines with blank cells. */
	TERMINAL_API void ScrollRegionDown(int32 TopRow, int32 BottomRow, int32 Count);

	/** Fill a region of cells with a given cell value. */
	TERMINAL_API void FillRegion(int32 AbsoluteRowStart, int32 ColumnStart, int32 AbsoluteRowEnd, int32 ColumnEnd, const FTerminalCell& Cell);

	/** Get text content from a range of buffer-absolute rows. */
	TERMINAL_API FString GetTextInRange(int32 StartRow, int32 StartColumn, int32 EndRow, int32 EndColumn) const;

	/** Clear all scrollback rows, keeping only the current viewport content. */
	TERMINAL_API void ClearScrollback();

	FTerminalCursor Cursor;

	/** Default cell used for clearing and new lines. */
	FTerminalCell DefaultCell;

	/** Alternate screen buffer support. */
	TERMINAL_API void ActivateAlternateBuffer();
	TERMINAL_API void DeactivateAlternateBuffer();
	bool IsAlternateBufferActive() const { return bAlternateBufferActive; }

private:

	/** Get the ring buffer index for an absolute row. */
	int32 GetRingIndex(int32 AbsoluteRow) const;

	/** Ensure a row exists at the given ring index with correct column count. */
	void EnsureRow(int32 RingIndex);

	int32 Columns = 80;
	int32 ViewportRows = 24;
	int32 ScrollbackLimit = 131072;

	/** Ring buffer of rows. Each row is a TArray<FTerminalCell> of Columns length. */
	TArray<TArray<FTerminalCell>> Rows;
	int32 RingStart = 0;
	int32 TotalRowsWritten = 0;

	/** Alternate buffer state. */
	bool bAlternateBufferActive = false;
	TArray<TArray<FTerminalCell>> SavedMainRows;
	int32 SavedRingStart = 0;
	int32 SavedTotalRowsWritten = 0;
	FTerminalCursor SavedCursor;
};
