// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Terminal color scheme loaded from a JSON file.
 *
 * Each scheme defines: default foreground/background, cursor color,
 * selection highlight color, and a 16-color ANSI palette.
 */
struct FTerminalColorScheme
{
	FString Name;
	FLinearColor DefaultForeground = FLinearColor(0.831f, 0.831f, 0.831f);
	FLinearColor DefaultBackground = FLinearColor(0.118f, 0.118f, 0.118f);
	FLinearColor CursorColor = FLinearColor(0.682f, 0.686f, 0.678f);
	FLinearColor SelectionColor = FLinearColor(0.149f, 0.310f, 0.471f);
	TArray<FLinearColor> Palette; // 16 ANSI colors (0-15)

	/** Create a default scheme. */
	static FTerminalColorScheme MakeDefault();

	/** Parse a scheme from a JSON string. Returns true on success. */
	static bool FromJSON(const FString& JSONString, FTerminalColorScheme& OutScheme);
};
