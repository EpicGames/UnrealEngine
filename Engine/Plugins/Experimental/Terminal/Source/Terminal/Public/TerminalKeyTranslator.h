// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FKeyEvent;

namespace UE::Terminal
{
	/** Options that influence key-to-byte translation. */
	struct FKeyTranslationOptions
	{
		/** DECCKM: when true, unmodified cursor / Home / End keys use SS3 sequences (ESC O x) instead of CSI (ESC [ x). */
		bool bApplicationCursorKeys = false;
	};

	/**
	 * Translate a Slate key event to the byte sequence a VT-compatible shell expects.
	 *
	 * Covers:
	 *  - Ctrl+A..Z via the `ch & 0x1F` bitmask.
	 *  - Ctrl+Space / [ / \ / ] / / / ? control-byte mappings.
	 *  - Enter, Tab (plus Shift+Tab = CSI Z), Escape, BackSpace (plus Ctrl/Alt variants).
	 *  - Arrows, Home, End, Insert, Delete, PageUp, PageDown, F1..F12 with the xterm modifier parameter
	 *    `1 + (Shift?1:0) + (Alt?2:0) + (Ctrl?4:0)`.
	 *  - Alt+<printable> readline meta: `ESC <byte>`. Guarded against AltGr (LeftCtrl + RightAlt on Windows).
	 *
	 * Returns an empty array when the key has no terminal meaning: pure modifier presses (Shift/Ctrl/Alt alone),
	 * unmapped keys, and printable characters that belong to the OnKeyChar path.
	 *
	 * Non-goals: Kitty keyboard protocol, DECKPAM (application keypad), C1 8-bit control sequences.
	 * Add with concrete motivation.
	 */
	TERMINAL_API TArray<uint8> TranslateKeyToBytes(const FKeyEvent& KeyEvent, const FKeyTranslationOptions& Options);
}
