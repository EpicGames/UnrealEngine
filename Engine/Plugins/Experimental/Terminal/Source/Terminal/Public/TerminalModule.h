// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Widgets/Docking/SDockTab.h"

class STerminal;

class FTerminalModule : public IModuleInterface
{
public:

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

private:

	TSharedRef<SDockTab> SpawnTerminalTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Returns false (vetoes close) if any STerminal is producing output and the user declines to close. */
	bool HandleCanCloseEditor();

	FDelegateHandle CanCloseEditorHandle;
};
