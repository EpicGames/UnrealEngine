// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "TerminalSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta=(DisplayName="Terminal"))
class UTerminalSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	/** Path to the shell executable. Leave empty to use the system default (COMSPEC on Windows, SHELL on Unix). */
	UPROPERTY(Config, EditAnywhere, Category="Terminal")
	FString ShellExecutablePath;

	/** Font filename stem (without extension) as found in the system Fonts directory, e.g. "consola" for Consolas. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal")
	FString FontFamily = TEXT("CascadiaMono");

	/** Font size in points. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal", meta=(ClampMin=6, ClampMax=72))
	int32 FontSize = 10;

	/** Maximum number of scrollback rows retained by the terminal. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal", meta=(ClampMin=0, ClampMax=1000000))
	int32 ScrollbackLimit = 131072;

	/** Name of the color scheme to use. Must match a JSON file in Config/ColorSchemes/. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal")
	FString ColorSchemeName = TEXT("Default");

	/** Commands to execute automatically when a new terminal window is created. Each entry is sent as a separate command. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal")
	TArray<FString> StartupCommands;

	/** When true, prompt for confirmation if the editor is closed while a terminal session is producing output. */
	UPROPERTY(Config, EditAnywhere, Category="Terminal|Activity")
	bool bPreventCloseDuringActivity = true;

	/** Seconds of silence after which a terminal is no longer considered active. Floor is 1.0s to stay above realistic tick jitter (frame hitches, GC pauses). */
	UPROPERTY(Config, EditAnywhere, Category="Terminal|Activity", meta=(ClampMin=1.0, ClampMax=60.0, EditCondition="bPreventCloseDuringActivity"))
	float ActivityTimeoutSeconds = 5.0f;
};
