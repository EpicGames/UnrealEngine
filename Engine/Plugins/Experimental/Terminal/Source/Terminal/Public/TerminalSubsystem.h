// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "TerminalColorScheme.h"

#include "TerminalSubsystem.generated.h"

/**
 * Editor subsystem that manages terminal sessions and color schemes.
 */
UCLASS(MinimalAPI)
class UTerminalSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	//~ Begin USubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface

	/** Get the active color scheme based on settings. */
	FTerminalColorScheme GetActiveColorScheme() const;

	/** Get a color scheme by name. Returns the default if not found. */
	FTerminalColorScheme GetColorScheme(const FString& Name) const;

	/** Reload color schemes from disk. */
	void ReloadColorSchemes();

private:

	/** Loaded color schemes indexed by name. */
	TMap<FString, FTerminalColorScheme> ColorSchemes;

	/** The hardcoded default scheme used as fallback. */
	FTerminalColorScheme DefaultScheme;
};
