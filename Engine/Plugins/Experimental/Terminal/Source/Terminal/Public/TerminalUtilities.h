// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace TerminalUtilities
{

/** Returns true for characters that the primary monospace font will render with correct cell-width advance. */
bool IsMonospaceSafe(TCHAR Character);

/** Resolve a font family name to an absolute .ttf path in the system Fonts directory. Returns empty if not found. */
FString ResolveSystemFontPath(const FString& FontFamily);

/** Platform-appropriate monospace font fallback names, tried in order. */
TArrayView<const TCHAR* const> GetFallbackFontNames();

/** Platform-appropriate symbol font names (box drawing, arrows, math, etc.), tried in order. */
TArrayView<const TCHAR* const> GetSymbolFontNames();

/** Platform-appropriate emoji font names, tried in order. */
TArrayView<const TCHAR* const> GetEmojiFontNames();

/** Engine-bundled monospace font path, used as final fallback when no system font is found. */
FString GetEngineFallbackFontPath();

/** Parse a hex color string (with or without leading '#') into a linear color. Returns white on invalid input. */
FLinearColor ParseHexColor(const FString& HexString);

} // namespace TerminalUtilities
