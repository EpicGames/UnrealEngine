// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FTerminalBuffer;

/**
 * VT/ANSI escape sequence parser implemented as a state machine.
 *
 * Follows the Paul Williams / vt100.net canonical parser model with 14 states,
 * "anywhere" transitions (ESC/CAN/SUB), and proper C0 mid-sequence execution.
 *
 * Processes a byte buffer of UTF-8 text containing VT sequences and
 * applies the resulting actions to an FTerminalBuffer.
 */
class FVTParser
{
public:

	TERMINAL_API FVTParser();

	/** Bind the parser to a terminal buffer. Must be called before Parse(). */
	TERMINAL_API void SetBuffer(FTerminalBuffer* InBuffer);

	/** Parse a chunk of UTF-8 bytes. */
	TERMINAL_API void Parse(const uint8* Data, int32 Length);

	/** Reset the parser and SGR state to defaults. */
	TERMINAL_API void Reset();

	/** Get the current parser state as a debug string. */
	TERMINAL_API FString GetDebugStateName() const;

	/** Initialize default tab stops (every 8 columns). */
	TERMINAL_API void InitializeTabStops();

	/** Current SGR state applied to newly written cells. */
	FColor CurrentForeground;
	FColor CurrentBackground;
	uint8 CurrentAttributes = 0;

	/** 16-color ANSI palette (indices 0-15). Initialized with defaults, overridden by color scheme. */
	FColor AnsiPalette[16];

	/** Scroll region (viewport-relative, inclusive). -1 means default (full viewport). */
	int32 ScrollRegionTop = -1;
	int32 ScrollRegionBottom = -1;

	/** Mouse tracking mode set via DECSET 1000/1002/1003. Encoding format controlled by mode 1006 (`bSGRMouseEncoding`). */
	enum class EMouseTrackingMode : uint8 { None, Normal, ButtonEvent, Any };

	/** Mode flags. */
	bool bAutoWrap = true;
	bool bCursorVisible = true;
	bool bApplicationCursorKeys = false;
	bool bBracketedPaste = false;
	bool bSynchronizedOutput = false;
	bool bSGRMouseEncoding = false;
	EMouseTrackingMode MouseTrackingMode = EMouseTrackingMode::None;

	/** Tab stops. */
	TArray<bool> TabStops;

	/** Window title set via OSC. */
	FString WindowTitle;

	/** Response buffer  - bytes to send back to the PTY (DA, DSR replies). Drained by STerminal::OnTick. */
	TArray<uint8> ResponseBuffer;

private:

	/** Parser states per the Williams/vt100.net state machine. */
	enum class EParserState : uint8
	{
		Ground,
		Escape,
		EscapeIntermediate,
		CSIEntry,
		CSIParam,
		CSIIntermediate,
		CSIIgnore,
		OSCString,
		DCSEntry,
		DCSParam,
		DCSIntermediate,
		DCSPassthrough,
		DCSIgnore,
		SOSPMAPCString,
	};

	EParserState State = EParserState::Ground;

	/** CSI/DCS parameter accumulation. */
	TArray<int32> CSIParams;
	FString CSIIntermediateChars;
	FString CurrentParam;

	/** CSI private parameter prefix character ('<', '=', '>', '?'), or '\0' for none. */
	TCHAR CSIPrivatePrefix = 0;

	/** Escape sequence intermediate characters accumulator. */
	FString EscapeIntermediateChars;

	/** OSC accumulation. */
	FString OSCPayload;

	FTerminalBuffer* Buffer = nullptr;

	/** UTF-8 decoder state. */
	uint32 UTF8Codepoint = 0;
	int32 UTF8Remaining = 0;
	uint32 UTF8MinCodepoint = 0; // Minimum valid codepoint for current sequence length (overlong detection).

	/** Process a single decoded Unicode codepoint. */
	void ProcessCodepoint(uint32 Codepoint);

	/** Reset CSI/DCS parameter accumulation state (the spec's "clear" action). */
	void ClearSequenceState();

	/** Execute a C0 control character in Ground context. Returns true if handled. */
	bool TryExecuteC0(uint32 Codepoint);

	/** State handlers. */
	void HandleGround(uint32 Codepoint);
	void HandleEscape(uint32 Codepoint);
	void HandleEscapeIntermediate(uint32 Codepoint);
	void HandleCSIEntry(uint32 Codepoint);
	void HandleCSIParam(uint32 Codepoint);
	void HandleCSIIntermediate(uint32 Codepoint);
	void HandleCSIIgnore(uint32 Codepoint);
	void HandleOSCString(uint32 Codepoint);
	void HandleDCSEntry(uint32 Codepoint);
	void HandleDCSParam(uint32 Codepoint);
	void HandleDCSIntermediate(uint32 Codepoint);
	void HandleDCSPassthrough(uint32 Codepoint);
	void HandleDCSIgnore(uint32 Codepoint);
	void HandleSOSPMAPCString(uint32 Codepoint);

	/** Execute a completed CSI sequence. */
	void ExecuteCSI(uint32 FinalByte);

	/** Execute an SGR (Select Graphic Rendition) sequence. */
	void ExecuteSGR();

	/** Execute a DECSET/DECRST private mode sequence. */
	void ExecutePrivateMode(uint32 FinalByte);

	/** Execute a completed escape sequence with intermediates. */
	void ExecuteEscapeSequence(uint32 FinalByte);

	/** Execute a completed OSC sequence. */
	void ExecuteOSC();

	/** Write a printable character at the cursor position. */
	void PutCharacter(TCHAR Character);

	/** Check if a codepoint is zero-width (combining marks, variation selectors, etc.). */
	static bool IsZeroWidthCodepoint(uint32 Codepoint);

	/** Advance the cursor to the next line, scrolling if necessary. */
	void LineFeed();

	/** Get the effective scroll region boundaries. */
	int32 GetEffectiveScrollTop() const;
	int32 GetEffectiveScrollBottom() const;

	/** Helper to get a CSI parameter with a default value. */
	int32 GetCSIParam(int32 Index, int32 Default) const;

	/** Get a color from the 256-color palette. Indices 0-15 use AnsiPalette. */
	FColor Get256Color(int32 Index) const;
};
