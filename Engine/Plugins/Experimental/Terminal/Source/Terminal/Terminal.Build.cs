// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Terminal : ModuleRules
	{
		public Terminal(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"EditorSubsystem",
					"Engine",
					"InputCore",
					"Json",
					"JsonUtilities",
					"MainFrame",
					"Projects",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
					"WorkspaceMenuStructure",
				}
			);

			// forkpty() on Linux lives in libutil.
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
			{
				PublicSystemLibraries.Add("util");
			}
		}
	}
}
