// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "ITerminalSession.h"
#include "STerminal.h"
#include "TerminalBuffer.h"
#include "VTParser.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/App.h"
#include "Widgets/Layout/SScrollBar.h"

#if WITH_DEV_AUTOMATION_TESTS

// -- Helpers --

/** Feed a string as UTF-8 bytes into the parser. */
static void ParseTerminalString(FVTParser& Parser, const FString& Text)
{
	const FTCHARToUTF8 Utf8(*Text);
	Parser.Parse(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
}

/** Synthesize a keyboard event.
 *  Only the LEFT modifier side is set to avoid the AltGr pattern (LeftCtrl + RightAlt on Windows)
 *  being accidentally triggered when both bCtrl and bAlt are true. */
static FKeyEvent MakeKeyEvent(FKey Key, bool bShift = false, bool bCtrl = false, bool bAlt = false, bool bCmd = false)
{
	FModifierKeysState Modifiers(bShift, false, bCtrl, false, bAlt, false, bCmd, false, false);
	return FKeyEvent(Key, Modifiers, 0, false, 0, 0);
}

/** Synthesize a keyboard event with the platform's copy/paste modifier held (Cmd on macOS, Ctrl elsewhere). */
static FKeyEvent MakeCopyPasteKeyEvent(FKey Key)
{
#if PLATFORM_MAC
	return MakeKeyEvent(Key, /*bShift*/ false, /*bCtrl*/ false, /*bAlt*/ false, /*bCmd*/ true);
#else
	return MakeKeyEvent(Key, /*bShift*/ false, /*bCtrl*/ true);
#endif
}

/** Minimal recording ITerminalSession implementation for widget-level tests. */
class FFakeTerminalSession : public ITerminalSession
{
public:
	TArray<uint8> WrittenBytes;

	virtual bool Create(const FString&, const FString&, int32, int32) override { return true; }
	virtual void Shutdown() override {}
	virtual void WriteInput(TArrayView<const uint8> Data) override
	{
		WrittenBytes.Append(Data.GetData(), Data.Num());
	}
	virtual void Resize(int32, int32) override {}
	virtual TArray<uint8> ConsumeOutput() override { return {}; }
	virtual bool IsRunning() const override { return true; }
};

/** Shared empty button set for synthesized pointer events. Static to avoid dangling references  - FPointerEvent stores a const TSet<FKey>& internally. */
static const TSet<FKey> EmptyPressedButtons;

/** Synthesize a mouse button event at a local pixel position. */
static FPointerEvent MakeMouseEvent(const FVector2D& ScreenPosition, FKey Button = EKeys::LeftMouseButton, bool bShift = false)
{
	FModifierKeysState Modifiers(bShift, bShift, false, false, false, false, false, false, false);
	return FPointerEvent(
		0,
		ScreenPosition,
		ScreenPosition,
		EmptyPressedButtons,
		Button,
		0.0f,
		Modifiers);
}

/** Synthesize a mouse wheel event. */
static FPointerEvent MakeWheelEvent(float WheelDelta, bool bCtrl = false)
{
	FModifierKeysState Modifiers(false, false, bCtrl, bCtrl, false, false, false, false, false);
	return FPointerEvent(
		0,
		FVector2D::ZeroVector,
		FVector2D::ZeroVector,
		EmptyPressedButtons,
		EKeys::Invalid,
		WheelDelta,
		Modifiers);
}

// -- Spec --

BEGIN_DEFINE_SPEC(FSTerminalSpec, "Terminal.STerminal",
	EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	TSharedPtr<STerminal> Terminal;

	/** Initialize the terminal's internal buffer and state without spawning a real PTY session. */
	void InitializeMinimalTerminal(int32 Columns = 80, int32 Rows = 24, int32 ScrollbackLimit = 100)
	{
		Terminal->ViewportColumns = Columns;
		Terminal->ViewportRows = Rows;
		Terminal->Buffer.Initialize(Columns, Rows, ScrollbackLimit);
		Terminal->Parser.SetBuffer(&Terminal->Buffer);
		Terminal->bInitialized = true;
		Terminal->CellWidth = 8.0f;
		Terminal->CellHeight = 16.0f;
	}

	/** Write text into the terminal buffer via the parser. */
	void WriteText(const FString& Text)
	{
		ParseTerminalString(Terminal->Parser, Text);
	}

	/** Generate enough lines to fill the viewport and create scrollback. */
	void GenerateScrollback(int32 LineCount = 50)
	{
		for (int32 Index = 0; Index < LineCount; ++Index)
		{
			WriteText(FString::Printf(TEXT("Line %d\r\n"), Index));
		}
	}

	/** Create a geometry matching the terminal's viewport dimensions. */
	FGeometry MakeGeometry() const
	{
		const float Width = Terminal->ViewportColumns * Terminal->CellWidth;
		const float Height = Terminal->ViewportRows * Terminal->CellHeight;
		return FGeometry::MakeRoot(FVector2D(Width, Height), FSlateLayoutTransform(FVector2D::ZeroVector));
	}

END_DEFINE_SPEC(FSTerminalSpec)

void FSTerminalSpec::Define()
{
	BeforeEach([this]()
	{
		Terminal = SNew(STerminal);
		InitializeMinimalTerminal();
	});

	AfterEach([this]()
	{
		Terminal.Reset();
	});

	// ===============================================================
	// Foundational: coordinate mapping and buffer math
	// ===============================================================

	Describe("Buffer Coordinate Mapping", [this]()
	{
		It("should map viewport rows correctly with scrollback", [this]()
		{
			GenerateScrollback();
			TestTrue(TEXT("has scrollback"), Terminal->Buffer.GetScrollbackLength() > 0);

			const int32 TopRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
			const int32 BottomRow = Terminal->Buffer.GetAbsoluteRow(Terminal->ViewportRows - 1, 0);
			TestEqual(TEXT("viewport spans ViewportRows"), BottomRow - TopRow, Terminal->ViewportRows - 1);
		});

		It("should shift rows with scroll offset", [this]()
		{
			GenerateScrollback();
			const int32 RowAtOffset0 = Terminal->Buffer.GetAbsoluteRow(0, 0);
			const int32 RowAtOffset5 = Terminal->Buffer.GetAbsoluteRow(0, 5);
			TestEqual(TEXT("offset shifts by 5"), RowAtOffset0 - RowAtOffset5, 5);
		});

		It("should compute correct scrollback length", [this]()
		{
			TestEqual(TEXT("no scrollback initially"), Terminal->Buffer.GetScrollbackLength(), 0);

			GenerateScrollback();
			const int32 ExpectedScrollback = Terminal->Buffer.GetTotalRows() - Terminal->ViewportRows;
			TestEqual(TEXT("scrollback = total - viewport"), Terminal->Buffer.GetScrollbackLength(), ExpectedScrollback);
		});
	});

	Describe("PixelToCell", [this]()
	{
		It("should map top-left pixel to cell (0, topRow)", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(0.0f, 0.0f));
			TestEqual(TEXT("column"), Cell.X, 0);
			TestEqual(TEXT("row"), Cell.Y, Terminal->Buffer.GetAbsoluteRow(0, Terminal->ScrollOffset));
		});

		It("should map bottom-right pixel to last cell", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const float MaxX = (Terminal->ViewportColumns - 1) * Terminal->CellWidth + 1.0f;
			const float MaxY = (Terminal->ViewportRows - 1) * Terminal->CellHeight + 1.0f;
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(MaxX, MaxY));
			TestEqual(TEXT("column"), Cell.X, Terminal->ViewportColumns - 1);
			TestEqual(TEXT("row"), Cell.Y, Terminal->Buffer.GetAbsoluteRow(Terminal->ViewportRows - 1, Terminal->ScrollOffset));
		});

		It("should clamp negative coordinates to (0, topRow)", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(-100.0f, -100.0f));
			TestEqual(TEXT("column"), Cell.X, 0);
			TestEqual(TEXT("row"), Cell.Y, Terminal->Buffer.GetAbsoluteRow(0, Terminal->ScrollOffset));
		});

		It("should clamp overflow coordinates to last cell", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(9999.0f, 9999.0f));
			TestEqual(TEXT("column"), Cell.X, Terminal->ViewportColumns - 1);
			TestEqual(TEXT("row"), Cell.Y, Terminal->Buffer.GetAbsoluteRow(Terminal->ViewportRows - 1, Terminal->ScrollOffset));
		});

		It("should map mid-cell pixel to correct column", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const float MidX = 3.0f * Terminal->CellWidth + Terminal->CellWidth * 0.5f;
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(MidX, 0.0f));
			TestEqual(TEXT("column"), Cell.X, 3);
		});

		It("should account for scroll offset", [this]()
		{
			GenerateScrollback();
			Terminal->ScrollOffset = 5;
			const FGeometry Geometry = MakeGeometry();
			const FIntPoint Cell = Terminal->PixelToCell(Geometry, FVector2D(0.0f, 0.0f));
			TestEqual(TEXT("scrolled row"), Cell.Y, Terminal->Buffer.GetAbsoluteRow(0, 5));
		});
	});

	// ===============================================================
	// Selection state: querying, text extraction, clearing
	// ===============================================================

	Describe("IsCellSelected", [this]()
	{
		It("should return false when no selection exists", [this]()
		{
			TestFalse(TEXT("no selection"), Terminal->IsCellSelected(0, 0));
		});

		It("should detect single-row selection", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(2, 0);
			Terminal->SelectionActive = FIntPoint(5, 0);
			Terminal->bHasSelection = true;

			TestFalse(TEXT("before start"), Terminal->IsCellSelected(0, 1));
			TestTrue(TEXT("at start"), Terminal->IsCellSelected(0, 2));
			TestTrue(TEXT("middle"), Terminal->IsCellSelected(0, 3));
			TestTrue(TEXT("at end"), Terminal->IsCellSelected(0, 5));
			TestFalse(TEXT("after end"), Terminal->IsCellSelected(0, 6));
		});

		It("should detect multi-row selection", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(5, 1);
			Terminal->SelectionActive = FIntPoint(3, 3);
			Terminal->bHasSelection = true;

			TestFalse(TEXT("row 0"), Terminal->IsCellSelected(0, 5));
			TestFalse(TEXT("row 1, col 4"), Terminal->IsCellSelected(1, 4));
			TestTrue(TEXT("row 1, col 5"), Terminal->IsCellSelected(1, 5));
			TestTrue(TEXT("row 1, col 79"), Terminal->IsCellSelected(1, 79));
			TestTrue(TEXT("row 2, col 0"), Terminal->IsCellSelected(2, 0));
			TestTrue(TEXT("row 2, col 79"), Terminal->IsCellSelected(2, 79));
			TestTrue(TEXT("row 3, col 0"), Terminal->IsCellSelected(3, 0));
			TestTrue(TEXT("row 3, col 3"), Terminal->IsCellSelected(3, 3));
			TestFalse(TEXT("row 3, col 4"), Terminal->IsCellSelected(3, 4));
			TestFalse(TEXT("row 4"), Terminal->IsCellSelected(4, 0));
		});

		It("should handle reverse selection (anchor after active)", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(5, 2);
			Terminal->SelectionActive = FIntPoint(2, 0);
			Terminal->bHasSelection = true;

			TestTrue(TEXT("start of reverse"), Terminal->IsCellSelected(0, 2));
			TestTrue(TEXT("middle row"), Terminal->IsCellSelected(1, 0));
			TestTrue(TEXT("end of reverse"), Terminal->IsCellSelected(2, 5));
			TestFalse(TEXT("outside"), Terminal->IsCellSelected(2, 6));
		});

		It("should handle degenerate single-cell selection", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(3, 2);
			Terminal->SelectionActive = FIntPoint(3, 2);
			Terminal->bHasSelection = true;

			TestTrue(TEXT("exact cell"), Terminal->IsCellSelected(2, 3));
			TestFalse(TEXT("adjacent cell"), Terminal->IsCellSelected(2, 4));
		});

		It("should work during active drag (bSelecting)", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(0, 0);
			Terminal->SelectionActive = FIntPoint(10, 0);
			Terminal->bSelecting = true;
			Terminal->bHasSelection = false;

			TestTrue(TEXT("cell within drag"), Terminal->IsCellSelected(0, 5));
		});
	});

	Describe("GetSelectedText", [this]()
	{
		It("should return empty when no selection", [this]()
		{
			TestEqual(TEXT("empty"), Terminal->GetSelectedText(), FString());
		});

		It("should return selected text from a single row", [this]()
		{
			WriteText(TEXT("Hello World"));
			const int32 AbsoluteRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
			Terminal->SelectionAnchor = FIntPoint(0, AbsoluteRow);
			Terminal->SelectionActive = FIntPoint(4, AbsoluteRow);
			Terminal->bHasSelection = true;

			TestEqual(TEXT("selected text"), Terminal->GetSelectedText(), TEXT("Hello"));
		});

		It("should return text across multiple rows", [this]()
		{
			WriteText(TEXT("AAAA\r\nBBBB\r\nCCCC"));
			const int32 Row0 = Terminal->Buffer.GetAbsoluteRow(0, 0);
			const int32 Row1 = Terminal->Buffer.GetAbsoluteRow(1, 0);
			Terminal->SelectionAnchor = FIntPoint(2, Row0);
			Terminal->SelectionActive = FIntPoint(1, Row1);
			Terminal->bHasSelection = true;

			const FString Selected = Terminal->GetSelectedText();
			TestTrue(TEXT("starts with AA"), Selected.StartsWith(TEXT("AA")));
			TestTrue(TEXT("ends with BB"), Selected.EndsWith(TEXT("BB")));
		});

		It("should return text from scrollback", [this]()
		{
			GenerateScrollback();
			const int32 ScrollbackRow = Terminal->Buffer.GetAbsoluteRow(0, 10);
			Terminal->SelectionAnchor = FIntPoint(0, ScrollbackRow);
			Terminal->SelectionActive = FIntPoint(3, ScrollbackRow);
			Terminal->bHasSelection = true;

			TestTrue(TEXT("non-empty from scrollback"), Terminal->GetSelectedText().Len() > 0);
		});
	});

	Describe("ClearSelection", [this]()
	{
		It("should reset all selection state", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(1, 2);
			Terminal->SelectionActive = FIntPoint(3, 4);
			Terminal->bHasSelection = true;
			Terminal->bSelecting = true;
			Terminal->SelectionAutoScrollDirection = 3;

			Terminal->ClearSelection();

			TestFalse(TEXT("bHasSelection"), Terminal->bHasSelection);
			TestFalse(TEXT("bSelecting"), Terminal->bSelecting);
			TestEqual(TEXT("auto-scroll cleared"), Terminal->SelectionAutoScrollDirection, 0);
			TestEqual(TEXT("anchor reset"), Terminal->SelectionAnchor, FIntPoint(-1, -1));
			TestEqual(TEXT("active reset"), Terminal->SelectionActive, FIntPoint(-1, -1));
		});
	});

	// ===============================================================
	// Click selection: single, double (word), triple (line)
	// ===============================================================

	Describe("Single Click", [this]()
	{
		It("should clear existing selection on single click", [this]()
		{
			Terminal->SelectionAnchor = FIntPoint(0, 0);
			Terminal->SelectionActive = FIntPoint(10, 0);
			Terminal->bHasSelection = true;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(50.0f, 50.0f)));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(FVector2D(50.0f, 50.0f)));

			TestFalse(TEXT("selection cleared"), Terminal->bHasSelection);
		});

		It("should degenerate to no selection when start equals end", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const FVector2D Position(10.0f, 10.0f);

			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(Position));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(Position));

			TestFalse(TEXT("no selection"), Terminal->bHasSelection);
		});
	});

	Describe("Double-Click Word Selection", [this]()
	{
		It("should select a word on double-click", [this]()
		{
			WriteText(TEXT("Hello World"));
			const FGeometry Geometry = MakeGeometry();

			// Position over 'W' in "World" (column 6).
			const FVector2D ClickPosition(6.0f * Terminal->CellWidth + 1.0f, 1.0f);

			// First click.
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(ClickPosition));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(ClickPosition));

			// Second click (double-click).
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(ClickPosition));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestEqual(TEXT("anchor col"), Terminal->SelectionAnchor.X, 6);
			TestEqual(TEXT("active col"), Terminal->SelectionActive.X, 10);
		});
	});

	Describe("Triple-Click Line Selection", [this]()
	{
		It("should select entire row on triple-click", [this]()
		{
			WriteText(TEXT("Hello World"));
			const FGeometry Geometry = MakeGeometry();
			const FVector2D ClickPosition(10.0f, 1.0f);

			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(ClickPosition));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(ClickPosition));
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(ClickPosition));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(ClickPosition));
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(ClickPosition));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestEqual(TEXT("anchor col 0"), Terminal->SelectionAnchor.X, 0);
			TestEqual(TEXT("active col last"), Terminal->SelectionActive.X, Terminal->ViewportColumns - 1);
		});
	});

	// ===============================================================
	// Drag selection
	// ===============================================================

	Describe("Drag Selection", [this]()
	{
		It("should create selection when dragging across cells", [this]()
		{
			WriteText(TEXT("Hello World"));
			const FGeometry Geometry = MakeGeometry();

			const FVector2D StartPosition(0.0f, 1.0f);
			const FVector2D EndPosition(4.0f * Terminal->CellWidth + 1.0f, 1.0f);

			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(StartPosition));
			TestTrue(TEXT("selecting"), Terminal->bSelecting);

			Terminal->OnMouseMove(Geometry, MakeMouseEvent(EndPosition));
			TestNotEqual(TEXT("active moved"), Terminal->SelectionActive.X, Terminal->SelectionAnchor.X);

			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(EndPosition));
			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestFalse(TEXT("no longer selecting"), Terminal->bSelecting);
		});

		It("should create multi-row selection when dragging vertically", [this]()
		{
			WriteText(TEXT("Line 0\r\nLine 1\r\nLine 2\r\n"));
			const FGeometry Geometry = MakeGeometry();

			const FVector2D StartPosition(2.0f * Terminal->CellWidth, 0.5f * Terminal->CellHeight);
			const FVector2D EndPosition(5.0f * Terminal->CellWidth, 2.5f * Terminal->CellHeight);

			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(StartPosition));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(EndPosition));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(EndPosition));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestTrue(TEXT("spans multiple rows"), Terminal->SelectionAnchor.Y != Terminal->SelectionActive.Y);
		});
	});

	// ===============================================================
	// Drag auto-scroll (dragging above/below viewport)
	// ===============================================================

	Describe("Drag Auto-Scroll", [this]()
	{
		BeforeEach([this]()
		{
			GenerateScrollback();
		});

		It("should set positive auto-scroll when dragging above viewport", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -20.0f)));

			TestTrue(TEXT("auto-scroll direction > 0"), Terminal->SelectionAutoScrollDirection > 0);
			TestTrue(TEXT("scroll offset increased"), Terminal->ScrollOffset > 0);
		});

		It("should set negative auto-scroll when dragging below viewport", [this]()
		{
			Terminal->ScrollOffset = 10;
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));

			const float BelowY = Terminal->ViewportRows * Terminal->CellHeight + 20.0f;
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, BelowY)));

			TestTrue(TEXT("auto-scroll direction < 0"), Terminal->SelectionAutoScrollDirection < 0);
			TestTrue(TEXT("scroll offset decreased"), Terminal->ScrollOffset < 10);
		});

		It("should set SelectionActive to top viewport row when dragging above", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -20.0f)));

			const int32 TopVisibleRow = Terminal->Buffer.GetAbsoluteRow(0, Terminal->ScrollOffset);
			TestEqual(TEXT("active at top of viewport"), Terminal->SelectionActive.Y, TopVisibleRow);
		});

		It("should set SelectionActive to bottom viewport row when dragging below", [this]()
		{
			Terminal->ScrollOffset = 10;
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));

			const float BelowY = Terminal->ViewportRows * Terminal->CellHeight + 20.0f;
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, BelowY)));

			const int32 BottomVisibleRow = Terminal->Buffer.GetAbsoluteRow(Terminal->ViewportRows - 1, Terminal->ScrollOffset);
			TestEqual(TEXT("active at bottom of viewport"), Terminal->SelectionActive.Y, BottomVisibleRow);
		});

		It("should preserve column during auto-scroll", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			const float TargetColumnX = 10.0f * Terminal->CellWidth + 1.0f;
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(TargetColumnX, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(TargetColumnX, -20.0f)));

			TestEqual(TEXT("column preserved"), Terminal->SelectionActive.X, 10);
		});

		It("should scale scroll speed with distance outside viewport", [this]()
		{
			const FGeometry Geometry = MakeGeometry();

			// Slightly outside: -5px = less than one cell height above.
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -5.0f)));
			const int32 SmallDirection = Terminal->SelectionAutoScrollDirection;

			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(FVector2D(10.0f, -5.0f)));

			// Far outside: -80px = multiple cell heights above.
			// Use a different click position to avoid double-click detection.
			Terminal->ScrollOffset = 0;
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(200.0f, 100.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -80.0f)));
			const int32 LargeDirection = Terminal->SelectionAutoScrollDirection;

			TestTrue(TEXT("far outside scrolls faster"), LargeDirection >= SmallDirection);
		});

		It("should clear auto-scroll when mouse returns inside viewport", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -20.0f)));
			TestNotEqual(TEXT("auto-scroll set"), Terminal->SelectionAutoScrollDirection, 0);

			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			TestEqual(TEXT("auto-scroll cleared"), Terminal->SelectionAutoScrollDirection, 0);
		});

		It("should clear auto-scroll on mouse button up", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -20.0f)));
			Terminal->OnMouseButtonUp(Geometry, MakeMouseEvent(FVector2D(10.0f, -20.0f)));

			TestEqual(TEXT("auto-scroll cleared"), Terminal->SelectionAutoScrollDirection, 0);
		});

		It("should clamp scroll offset when no scrollback available", [this]()
		{
			Terminal.Reset();
			Terminal = SNew(STerminal);
			InitializeMinimalTerminal();

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseButtonDown(Geometry, MakeMouseEvent(FVector2D(10.0f, 10.0f)));
			Terminal->OnMouseMove(Geometry, MakeMouseEvent(FVector2D(10.0f, -50.0f)));

			TestEqual(TEXT("offset stays 0"), Terminal->ScrollOffset, 0);
		});
	});

	// ===============================================================
	// Keyboard selection (Shift+Arrow)
	// ===============================================================

	Describe("Shift+Arrow Keyboard Selection", [this]()
	{
		BeforeEach([this]()
		{
			WriteText(TEXT("Line 0\r\nLine 1\r\nLine 2\r\nLine 3\r\n"));
			Terminal->Buffer.Cursor.Row = 2;
			Terminal->Buffer.Cursor.Column = 3;
		});

		// Vertical movement.

		It("should start selection at cursor on first Shift+Up", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Up, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			const int32 CursorAbsoluteRow = Terminal->Buffer.GetAbsoluteRow(2, 0);
			TestEqual(TEXT("anchor row"), Terminal->SelectionAnchor.Y, CursorAbsoluteRow);
			TestEqual(TEXT("anchor col"), Terminal->SelectionAnchor.X, 3);
			TestEqual(TEXT("active row moved up"), Terminal->SelectionActive.Y, CursorAbsoluteRow - 1);
		});

		It("should start selection at cursor on first Shift+Down", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Down, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			const int32 CursorAbsoluteRow = Terminal->Buffer.GetAbsoluteRow(2, 0);
			TestEqual(TEXT("active row moved down"), Terminal->SelectionActive.Y, CursorAbsoluteRow + 1);
		});

		It("should extend selection with multiple Shift+Up presses", [this]()
		{
			// Cursor at row 2  - press Shift+Up twice to reach row 0.
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Up, true));
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Up, true));

			const int32 CursorAbsoluteRow = Terminal->Buffer.GetAbsoluteRow(2, 0);
			TestEqual(TEXT("active moved 2 rows up"), Terminal->SelectionActive.Y, CursorAbsoluteRow - 2);
			TestEqual(TEXT("anchor unchanged"), Terminal->SelectionAnchor.Y, CursorAbsoluteRow);
		});

		It("should not extend selection past last row with Shift+Down", [this]()
		{
			Terminal->Buffer.Cursor.Row = 3;
			Terminal->Buffer.Cursor.Column = 0;
			const int32 LastRow = Terminal->Buffer.GetTotalRows() - 1;

			const FGeometry Geometry = MakeGeometry();
			for (int32 Index = 0; Index < 30; ++Index)
			{
				Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Down, true));
			}

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestTrue(TEXT("active at or before last row"), Terminal->SelectionActive.Y <= LastRow);
		});

		// Horizontal movement.

		It("should move Shift+Left within a row", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Left, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestEqual(TEXT("active col"), Terminal->SelectionActive.X, 2);
			TestEqual(TEXT("same row"), Terminal->SelectionActive.Y, Terminal->SelectionAnchor.Y);
		});

		It("should move Shift+Right within a row", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Right, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			TestEqual(TEXT("active col"), Terminal->SelectionActive.X, 4);
			TestEqual(TEXT("same row"), Terminal->SelectionActive.Y, Terminal->SelectionAnchor.Y);
		});

		// Line wrapping.

		It("should handle Shift+Left wrapping to previous row", [this]()
		{
			Terminal->Buffer.Cursor.Column = 0;
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Left, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			const int32 CursorAbsoluteRow = Terminal->Buffer.GetAbsoluteRow(2, 0);
			TestEqual(TEXT("active row wrapped up"), Terminal->SelectionActive.Y, CursorAbsoluteRow - 1);
			TestEqual(TEXT("active col at end"), Terminal->SelectionActive.X, Terminal->ViewportColumns - 1);
		});

		It("should handle Shift+Right wrapping to next row", [this]()
		{
			Terminal->Buffer.Cursor.Column = Terminal->ViewportColumns - 1;
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Right, true));

			TestTrue(TEXT("has selection"), Terminal->bHasSelection);
			const int32 CursorAbsoluteRow = Terminal->Buffer.GetAbsoluteRow(2, 0);
			TestEqual(TEXT("active row wrapped down"), Terminal->SelectionActive.Y, CursorAbsoluteRow + 1);
			TestEqual(TEXT("active col at start"), Terminal->SelectionActive.X, 0);
		});

		// Viewport scrolling.

		It("should scroll viewport when selection extends above visible area", [this]()
		{
			Terminal.Reset();
			Terminal = SNew(STerminal);
			InitializeMinimalTerminal(80, 5, 100);
			for (int32 Index = 0; Index < 20; ++Index)
			{
				WriteText(FString::Printf(TEXT("Line %d\r\n"), Index));
			}
			Terminal->ScrollOffset = 0;
			Terminal->Buffer.Cursor.Row = 0;
			Terminal->Buffer.Cursor.Column = 0;

			const FGeometry Geometry = FGeometry::MakeRoot(
				FVector2D(Terminal->ViewportColumns * Terminal->CellWidth, Terminal->ViewportRows * Terminal->CellHeight),
				FSlateLayoutTransform(FVector2D::ZeroVector));

			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Up, true));
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Up, true));

			TestTrue(TEXT("viewport scrolled"), Terminal->ScrollOffset > 0);
		});
	});

	// ===============================================================
	// Mouse wheel scrolling
	// ===============================================================

	Describe("Mouse Wheel Scrolling", [this]()
	{
		It("should scroll up on positive wheel delta", [this]()
		{
			GenerateScrollback();
			TestEqual(TEXT("initial scroll offset"), Terminal->ScrollOffset, 0);

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(1.0f));

			TestTrue(TEXT("scrolled up"), Terminal->ScrollOffset > 0);
		});

		It("should not scroll below zero", [this]()
		{
			Terminal->ScrollOffset = 0;
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(-1.0f));

			TestEqual(TEXT("clamped to zero"), Terminal->ScrollOffset, 0);
		});

		It("should not scroll above max scrollback", [this]()
		{
			GenerateScrollback();
			const int32 MaxScrollback = Terminal->Buffer.GetScrollbackLength();
			Terminal->ScrollOffset = MaxScrollback;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(100.0f));

			TestEqual(TEXT("clamped to max"), Terminal->ScrollOffset, MaxScrollback);
		});

		It("should preserve selection when scrolling", [this]()
		{
			GenerateScrollback();
			const int32 AbsoluteRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
			Terminal->SelectionAnchor = FIntPoint(0, AbsoluteRow);
			Terminal->SelectionActive = FIntPoint(5, AbsoluteRow);
			Terminal->bHasSelection = true;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(3.0f));

			TestTrue(TEXT("selection preserved"), Terminal->bHasSelection);
			TestEqual(TEXT("anchor unchanged"), Terminal->SelectionAnchor, FIntPoint(0, AbsoluteRow));
			TestEqual(TEXT("active unchanged"), Terminal->SelectionActive, FIntPoint(5, AbsoluteRow));
		});
	});

	// ===============================================================
	// Widget lifecycle: focus
	// ===============================================================

	Describe("Focus", [this]()
	{
		It("should set focus flag on focus received", [this]()
		{
			const FGeometry Geometry = MakeGeometry();
			Terminal->OnFocusReceived(Geometry, FFocusEvent());

			TestTrue(TEXT("has focus"), Terminal->bHasFocus);
		});

		It("should clear drag state on focus lost", [this]()
		{
			Terminal->bSelecting = true;
			Terminal->SelectionAutoScrollDirection = 3;

			Terminal->OnFocusLost(FFocusEvent());

			TestFalse(TEXT("selecting cleared"), Terminal->bSelecting);
			TestEqual(TEXT("auto-scroll cleared"), Terminal->SelectionAutoScrollDirection, 0);
			TestFalse(TEXT("focus flag"), Terminal->bHasFocus);
		});
	});

	// ===============================================================
	// Alternate buffer scrolling
	// ===============================================================

	Describe("Alternate Buffer Scrolling", [this]()
	{
		It("should not scroll in alternate screen buffer", [this]()
		{
			GenerateScrollback();
			WriteText(TEXT("\x1B[?1049h")); // Activate alternate buffer
			Terminal->ScrollOffset = 0;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(3.0f));

			TestEqual(TEXT("scroll offset stays 0"), Terminal->ScrollOffset, 0);
		});

		It("should scroll normally after leaving alternate buffer", [this]()
		{
			WriteText(TEXT("\x1B[?1049h")); // Activate
			WriteText(TEXT("\x1B[?1049l")); // Deactivate
			GenerateScrollback();

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(1.0f));

			TestTrue(TEXT("scrolled up"), Terminal->ScrollOffset > 0);
		});
	});

	// ===============================================================
	// Mouse tracking
	// ===============================================================

	Describe("Mouse Tracking", [this]()
	{
		It("should not adjust ScrollOffset when mouse tracking is active", [this]()
		{
			GenerateScrollback();
			WriteText(TEXT("\x1B[?1000h")); // Enable normal mouse tracking
			Terminal->ScrollOffset = 0;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(3.0f));

			TestEqual(TEXT("scroll offset stays 0"), Terminal->ScrollOffset, 0);
		});

		It("should resume normal scrolling when mouse tracking is disabled", [this]()
		{
			GenerateScrollback();
			WriteText(TEXT("\x1B[?1000h")); // Enable
			WriteText(TEXT("\x1B[?1000l")); // Disable

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnMouseWheel(Geometry, MakeWheelEvent(1.0f));

			TestTrue(TEXT("scrolled up"), Terminal->ScrollOffset > 0);
		});
	});

	Describe("Keyboard Input", [this]()
	{
		It("should copy the selection on the platform copy shortcut instead of sending ETX", [this]()
		{
			const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
			Terminal->Session = FakeSession;

			WriteText(TEXT("Hello"));
			const int32 AbsoluteRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
			Terminal->SelectionAnchor = FIntPoint(0, AbsoluteRow);
			Terminal->SelectionActive = FIntPoint(4, AbsoluteRow);
			Terminal->bHasSelection = true;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeCopyPasteKeyEvent(EKeys::C));

			TestFalse(TEXT("selection cleared"), Terminal->bHasSelection);
			TestEqual(TEXT("no bytes sent to the shell"), FakeSession->WrittenBytes.Num(), 0);

			FString ClipboardText;
			FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
			TestEqual(TEXT("clipboard roundtrip"), ClipboardText, FString(TEXT("Hello")));
		});

		It("should send ETX on Ctrl+C when there is no selection", [this]()
		{
			const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
			Terminal->Session = FakeSession;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::C, /*bShift*/ false, /*bCtrl*/ true));

			TestEqual(TEXT("byte count"), FakeSession->WrittenBytes.Num(), 1);
			if (FakeSession->WrittenBytes.Num() == 1)
			{
				TestEqual(TEXT("ETX"), FakeSession->WrittenBytes[0], static_cast<uint8>(0x03));
			}
		});

#if PLATFORM_MAC
		It("should send ETX on Ctrl+C even with a selection (copy is Cmd+C on macOS)", [this]()
		{
			const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
			Terminal->Session = FakeSession;

			WriteText(TEXT("Hello"));
			const int32 AbsoluteRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
			Terminal->SelectionAnchor = FIntPoint(0, AbsoluteRow);
			Terminal->SelectionActive = FIntPoint(4, AbsoluteRow);
			Terminal->bHasSelection = true;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::C, /*bShift*/ false, /*bCtrl*/ true));

			TestEqual(TEXT("byte count"), FakeSession->WrittenBytes.Num(), 1);
			if (FakeSession->WrittenBytes.Num() == 1)
			{
				TestEqual(TEXT("ETX"), FakeSession->WrittenBytes[0], static_cast<uint8>(0x03));
			}
			TestTrue(TEXT("selection preserved"), Terminal->bHasSelection);
		});
#endif

		It("should send 0x0F to the shell when Ctrl+O is pressed", [this]()
		{
			// This is the originally reported regression: Ctrl+O (nano's Write Out) must reach the PTY.
			const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
			Terminal->Session = FakeSession;

			const FGeometry Geometry = MakeGeometry();
			Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::O, false, true));

			TestEqual(TEXT("byte count"), FakeSession->WrittenBytes.Num(), 1);
			if (FakeSession->WrittenBytes.Num() == 1)
			{
				TestEqual(TEXT("SI (Ctrl+O)"), FakeSession->WrittenBytes[0], static_cast<uint8>(0x0F));
			}
		});

		It("should consume unmapped modified keys without writing to the session", [this]()
		{
			const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
			Terminal->Session = FakeSession;

			const FGeometry Geometry = MakeGeometry();
			const FReply Reply = Terminal->OnKeyDown(Geometry, MakeKeyEvent(EKeys::Gamepad_FaceButton_Bottom, false, true));

			TestTrue(TEXT("reply is handled"), Reply.IsEventHandled());
			TestEqual(TEXT("no bytes written"), FakeSession->WrittenBytes.Num(), 0);
		});
	});

	Describe("Shortcut Handlers", [this]()
	{
		Describe("HandleShiftArrowSelection", [this]()
		{
			It("should handle Shift+Up by starting a selection at the cursor", [this]()
			{
				TestFalse(TEXT("no selection initially"), Terminal->bHasSelection);

				const TOptional<FReply> Reply = Terminal->HandleShiftArrowSelection(MakeKeyEvent(EKeys::Up, /*bShift*/ true));

				TestTrue(TEXT("reply set"), Reply.IsSet());
				TestTrue(TEXT("selection started"), Terminal->bHasSelection);
			});

			It("should decline non-arrow keys", [this]()
			{
				const TOptional<FReply> Reply = Terminal->HandleShiftArrowSelection(MakeKeyEvent(EKeys::A, /*bShift*/ true));
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});

			It("should decline when Ctrl is also held", [this]()
			{
				const TOptional<FReply> Reply = Terminal->HandleShiftArrowSelection(MakeKeyEvent(EKeys::Up, /*bShift*/ true, /*bCtrl*/ true));
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});
		});

		Describe("HandleCopyShortcut", [this]()
		{
			It("should copy the selection and clear it when the copy modifier is held", [this]()
			{
				WriteText(TEXT("World"));
				const int32 AbsoluteRow = Terminal->Buffer.GetAbsoluteRow(0, 0);
				Terminal->SelectionAnchor = FIntPoint(0, AbsoluteRow);
				Terminal->SelectionActive = FIntPoint(4, AbsoluteRow);
				Terminal->bHasSelection = true;

				const TOptional<FReply> Reply = Terminal->HandleCopyShortcut(MakeKeyEvent(EKeys::C), /*bCopyPasteModifier*/ true);

				TestTrue(TEXT("reply set"), Reply.IsSet());
				TestFalse(TEXT("selection cleared"), Terminal->bHasSelection);

				FString ClipboardText;
				FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
				TestEqual(TEXT("clipboard roundtrip"), ClipboardText, FString(TEXT("World")));
			});

			It("should decline without a selection so the translator can send ETX", [this]()
			{
				TestFalse(TEXT("no selection"), Terminal->bHasSelection);

				const TOptional<FReply> Reply = Terminal->HandleCopyShortcut(MakeKeyEvent(EKeys::C), /*bCopyPasteModifier*/ true);
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});

			It("should decline when the copy modifier is not held", [this]()
			{
				Terminal->bHasSelection = true;
				const TOptional<FReply> Reply = Terminal->HandleCopyShortcut(MakeKeyEvent(EKeys::C), /*bCopyPasteModifier*/ false);
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});

			It("should decline for keys other than C", [this]()
			{
				Terminal->bHasSelection = true;
				const TOptional<FReply> Reply = Terminal->HandleCopyShortcut(MakeKeyEvent(EKeys::V), /*bCopyPasteModifier*/ true);
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});
		});

		Describe("HandlePasteShortcut", [this]()
		{
			It("should write clipboard contents to the session when the paste modifier is held", [this]()
			{
				const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
				Terminal->Session = FakeSession;

				FPlatformApplicationMisc::ClipboardCopy(TEXT("paste me"));

				const TOptional<FReply> Reply = Terminal->HandlePasteShortcut(MakeKeyEvent(EKeys::V), /*bCopyPasteModifier*/ true);

				TestTrue(TEXT("reply set"), Reply.IsSet());
				TestTrue(TEXT("some bytes written"), FakeSession->WrittenBytes.Num() > 0);
			});

			It("should wrap the paste in bracketed-paste markers when enabled", [this]()
			{
				const TSharedRef<FFakeTerminalSession> FakeSession = MakeShared<FFakeTerminalSession>();
				Terminal->Session = FakeSession;
				Terminal->Parser.bBracketedPaste = true;

				FPlatformApplicationMisc::ClipboardCopy(TEXT("x"));

				const TOptional<FReply> Reply = Terminal->HandlePasteShortcut(MakeKeyEvent(EKeys::V), /*bCopyPasteModifier*/ true);

				TestTrue(TEXT("reply set"), Reply.IsSet());
				const TArray<uint8> Expected = { 0x1B, '[', '2', '0', '0', '~', 'x', 0x1B, '[', '2', '0', '1', '~' };
				TestEqual(TEXT("byte sequence"), FakeSession->WrittenBytes, Expected);
			});

			It("should decline when the paste modifier is not held", [this]()
			{
				const TOptional<FReply> Reply = Terminal->HandlePasteShortcut(MakeKeyEvent(EKeys::V), /*bCopyPasteModifier*/ false);
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});

			It("should decline for keys other than V", [this]()
			{
				const TOptional<FReply> Reply = Terminal->HandlePasteShortcut(MakeKeyEvent(EKeys::C), /*bCopyPasteModifier*/ true);
				TestFalse(TEXT("reply unset"), Reply.IsSet());
			});
		});
	});

	// ===============================================================
	// Instance registry: drives FTerminalModule::HandleCanCloseEditor.
	// ===============================================================

	Describe("Instance Registry", [this]()
	{
		It("should include the constructed terminal", [this]()
		{
			const TArray<TSharedRef<STerminal>> Instances = STerminal::GetAllInstances();
			TestTrue(TEXT("registry contains the BeforeEach-constructed terminal"), Instances.ContainsByPredicate(
				[this](const TSharedRef<STerminal>& Entry) { return &Entry.Get() == Terminal.Get(); }));
		});

		It("should drop entries for destroyed terminals", [this]()
		{
			TSharedPtr<STerminal> Extra = SNew(STerminal);
			STerminal* ExtraRaw = Extra.Get();

			TArray<TSharedRef<STerminal>> Instances = STerminal::GetAllInstances();
			TestTrue(TEXT("extra terminal registered"), Instances.ContainsByPredicate(
				[ExtraRaw](const TSharedRef<STerminal>& Entry) { return &Entry.Get() == ExtraRaw; }));

			Extra.Reset();

			Instances = STerminal::GetAllInstances();
			TestFalse(TEXT("extra terminal pruned after destruction"), Instances.ContainsByPredicate(
				[ExtraRaw](const TSharedRef<STerminal>& Entry) { return &Entry.Get() == ExtraRaw; }));
		});

		It("should report fresh output as recent activity", [this]()
		{
			InitializeMinimalTerminal();
			const double Now = FApp::GetCurrentTime();
			Terminal->LastOutputTime = Now;

			constexpr double Window = 5.0;
			const double LastOutputTime = Terminal->GetLastOutputTime();
			const bool bActive = LastOutputTime > 0.0 && (Now - LastOutputTime) < Window;
			TestTrue(TEXT("zero-silence terminal counts as active"), bActive);
		});

		It("should report stale output as idle", [this]()
		{
			InitializeMinimalTerminal();
			const double Now = FApp::GetCurrentTime();
			constexpr double Window = 5.0;
			Terminal->LastOutputTime = Now - (Window + 1.0);

			const double LastOutputTime = Terminal->GetLastOutputTime();
			const bool bActive = LastOutputTime > 0.0 && (Now - LastOutputTime) < Window;
			TestFalse(TEXT("terminal silent past the window is idle"), bActive);
		});

		It("should report a terminal that has never produced output as idle", [this]()
		{
			InitializeMinimalTerminal();
			Terminal->LastOutputTime = 0.0;

			const double Now = FApp::GetCurrentTime();
			constexpr double Window = 5.0;
			const double LastOutputTime = Terminal->GetLastOutputTime();
			const bool bActive = LastOutputTime > 0.0 && (Now - LastOutputTime) < Window;
			TestFalse(TEXT("zero LastOutputTime never counts as active"), bActive);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
