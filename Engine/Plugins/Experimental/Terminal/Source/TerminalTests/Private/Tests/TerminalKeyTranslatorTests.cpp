// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "TerminalKeyTranslator.h"

#include "InputCoreTypes.h"
#include "Input/Events.h"

#if WITH_DEV_AUTOMATION_TESTS

// -- Helpers --

/** Synthesize a key event with explicit modifier state.
 *  Only the LEFT modifier side is set so that Ctrl+Alt does not accidentally match the AltGr pattern
 *  (LeftCtrl + RightAlt on Windows). Use MakeAltGrKeyEvent for AltGr cases. */
static FKeyEvent MakeTranslatorKeyEvent(FKey Key, bool bShift = false, bool bCtrl = false, bool bAlt = false)
{
	// FModifierKeysState ctor: LeftShift, RightShift, LeftCtrl, RightCtrl, LeftAlt, RightAlt, LeftCmd, RightCmd, CapsLock.
	const FModifierKeysState Modifiers(bShift, false, bCtrl, false, bAlt, false, false, false, false);
	return FKeyEvent(Key, Modifiers, 0, false, 0, 0);
}

/** Synthesize a key event matching the Windows AltGr modifier state (LeftCtrl + RightAlt). */
static FKeyEvent MakeAltGrKeyEvent(FKey Key)
{
	const FModifierKeysState Modifiers(
		/*LeftShift*/  false, /*RightShift*/  false,
		/*LeftCtrl*/   true,  /*RightCtrl*/   false,
		/*LeftAlt*/    false, /*RightAlt*/    true,
		/*LeftCmd*/    false, /*RightCmd*/    false,
		/*CapsLock*/   false);
	return FKeyEvent(Key, Modifiers, 0, false, 0, 0);
}

/** Format a byte array as a space-separated hex string so `TestEqual` failure messages are readable. */
static FString BytesToHex(const TArray<uint8>& Bytes)
{
	FString Result;
	for (const uint8 Byte : Bytes)
	{
		if (!Result.IsEmpty())
		{
			Result.AppendChar(TEXT(' '));
		}
		Result += FString::Printf(TEXT("%02X"), Byte);
	}
	return Result;
}

static TArray<uint8> Translate(const FKeyEvent& KeyEvent, bool bApplicationCursorKeys = false)
{
	UE::Terminal::FKeyTranslationOptions Options;
	Options.bApplicationCursorKeys = bApplicationCursorKeys;
	return UE::Terminal::TranslateKeyToBytes(KeyEvent, Options);
}

// -- Spec --

BEGIN_DEFINE_SPEC(FTerminalKeyTranslatorSpec, "Terminal.KeyTranslator",
	EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	void ExpectBytes(const TArray<uint8>& Actual, const TArray<uint8>& Expected, const FString& Label)
	{
		TestEqual(*FString::Printf(TEXT("%s (hex)"), *Label), BytesToHex(Actual), BytesToHex(Expected));
	}

END_DEFINE_SPEC(FTerminalKeyTranslatorSpec)

void FTerminalKeyTranslatorSpec::Define()
{
	Describe("Ctrl + letter", [this]()
	{
		It("should produce 0x01..0x1A for every letter", [this]()
		{
			// EKeys::A..Z are registered with single-character uppercase names.
			const FKey Letters[] = {
				EKeys::A, EKeys::B, EKeys::C, EKeys::D, EKeys::E, EKeys::F, EKeys::G, EKeys::H, EKeys::I,
				EKeys::J, EKeys::K, EKeys::L, EKeys::M, EKeys::N, EKeys::O, EKeys::P, EKeys::Q, EKeys::R,
				EKeys::S, EKeys::T, EKeys::U, EKeys::V, EKeys::W, EKeys::X, EKeys::Y, EKeys::Z
			};
			for (int32 Index = 0; Index < UE_ARRAY_COUNT(Letters); ++Index)
			{
				const FKeyEvent Event = MakeTranslatorKeyEvent(Letters[Index], /*bShift*/ false, /*bCtrl*/ true);
				const uint8 Expected = static_cast<uint8>(Index + 1); // A->1, B->2, ..., Z->26
				ExpectBytes(Translate(Event), { Expected },
					FString::Printf(TEXT("Ctrl+%s"), *Letters[Index].ToString()));
			}
		});

		It("should produce 0x0F for Ctrl+O specifically (the originally reported regression)", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::O, false, true)), { 0x0F }, TEXT("Ctrl+O"));
		});
	});

	Describe("Ctrl + symbol", [this]()
	{
		It("should map Space/[/\\/]/ / ? to their control bytes", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::SpaceBar, false, true)),    { 0x00 }, TEXT("Ctrl+Space"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::LeftBracket, false, true)), { 0x1B }, TEXT("Ctrl+["));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Backslash, false, true)),   { 0x1C }, TEXT("Ctrl+\\"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::RightBracket, false, true)),{ 0x1D }, TEXT("Ctrl+]"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Slash, false, true)),       { 0x1F }, TEXT("Ctrl+/"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Slash, true, true)),        { 0x7F }, TEXT("Ctrl+?"));
		});
	});

	Describe("Named keys", [this]()
	{
		It("should map Enter / Tab / Escape / BackSpace unmodified", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Enter)),     { 0x0D }, TEXT("Enter"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Tab)),       { 0x09 }, TEXT("Tab"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Escape)),    { 0x1B }, TEXT("Escape"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::BackSpace)), { 0x7F }, TEXT("BackSpace"));
		});

		It("should send CSI Z for Shift+Tab", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Tab, true)), { 0x1B, '[', 'Z' }, TEXT("Shift+Tab"));
		});

		It("should send BS (0x08) for Ctrl+BackSpace and ESC DEL for Alt+BackSpace", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::BackSpace, false, true)),         { 0x08 },        TEXT("Ctrl+BackSpace"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::BackSpace, false, false, true)),  { 0x1B, 0x7F },  TEXT("Alt+BackSpace"));
		});
	});

	Describe("Arrows", [this]()
	{
		It("should send ANSI CSI sequences in non-application mode", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Up)),    { 0x1B, '[', 'A' }, TEXT("Up"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Down)),  { 0x1B, '[', 'B' }, TEXT("Down"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Right)), { 0x1B, '[', 'C' }, TEXT("Right"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Left)),  { 0x1B, '[', 'D' }, TEXT("Left"));
		});

		It("should send SS3 sequences in application cursor mode", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Up), true),    { 0x1B, 'O', 'A' }, TEXT("Up (app)"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Down), true),  { 0x1B, 'O', 'B' }, TEXT("Down (app)"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Right), true), { 0x1B, 'O', 'C' }, TEXT("Right (app)"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Left), true),  { 0x1B, 'O', 'D' }, TEXT("Left (app)"));
		});

		It("should send CSI with modifier parameter for modified arrows regardless of app mode", [this]()
		{
			// Ctrl+Left: modifier = 1 + 4 = 5.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Left, false, true)),       { 0x1B, '[', '1', ';', '5', 'D' }, TEXT("Ctrl+Left"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Left, false, true), true), { 0x1B, '[', '1', ';', '5', 'D' }, TEXT("Ctrl+Left (app)"));
			// Shift+Right: modifier = 1 + 1 = 2.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Right, true)),             { 0x1B, '[', '1', ';', '2', 'C' }, TEXT("Shift+Right"));
			// Alt+Up: modifier = 1 + 2 = 3.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Up, false, false, true)),  { 0x1B, '[', '1', ';', '3', 'A' }, TEXT("Alt+Up"));
			// Ctrl+Shift+Alt+Down: modifier = 1 + 1 + 2 + 4 = 8.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Down, true, true, true)),  { 0x1B, '[', '1', ';', '8', 'B' }, TEXT("Ctrl+Shift+Alt+Down"));
		});
	});

	Describe("Home / End / Insert / Delete / PageUp / PageDown", [this]()
	{
		It("should send their canonical sequences unmodified", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Home)),     { 0x1B, '[', 'H' },      TEXT("Home"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::End)),      { 0x1B, '[', 'F' },      TEXT("End"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Insert)),   { 0x1B, '[', '2', '~' }, TEXT("Insert"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Delete)),   { 0x1B, '[', '3', '~' }, TEXT("Delete"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::PageUp)),   { 0x1B, '[', '5', '~' }, TEXT("PageUp"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::PageDown)), { 0x1B, '[', '6', '~' }, TEXT("PageDown"));
		});

		It("should honor DECCKM for Home and End but not for tilde-terminated keys", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Home), true),   { 0x1B, 'O', 'H' },      TEXT("Home (app)"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::End), true),    { 0x1B, 'O', 'F' },      TEXT("End (app)"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Insert), true), { 0x1B, '[', '2', '~' }, TEXT("Insert (app unchanged)"));
		});

		It("should encode modifier parameters", [this]()
		{
			// Ctrl+Home: modifier = 5.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Home, false, true)),     { 0x1B, '[', '1', ';', '5', 'H' },      TEXT("Ctrl+Home"));
			// Shift+End: modifier = 2.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::End, true)),             { 0x1B, '[', '1', ';', '2', 'F' },      TEXT("Shift+End"));
			// Ctrl+Delete: modifier = 5.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Delete, false, true)),   { 0x1B, '[', '3', ';', '5', '~' },      TEXT("Ctrl+Delete"));
			// Shift+PageUp: modifier = 2.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::PageUp, true)),          { 0x1B, '[', '5', ';', '2', '~' },      TEXT("Shift+PageUp"));
		});
	});

	Describe("Function keys", [this]()
	{
		It("should send SS3 sequences for F1..F4 unmodified", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F1)), { 0x1B, 'O', 'P' }, TEXT("F1"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F2)), { 0x1B, 'O', 'Q' }, TEXT("F2"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F3)), { 0x1B, 'O', 'R' }, TEXT("F3"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F4)), { 0x1B, 'O', 'S' }, TEXT("F4"));
		});

		It("should send tilde-terminated sequences for F5..F12 unmodified", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F5)),  { 0x1B, '[', '1', '5', '~' }, TEXT("F5"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F6)),  { 0x1B, '[', '1', '7', '~' }, TEXT("F6"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F7)),  { 0x1B, '[', '1', '8', '~' }, TEXT("F7"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F8)),  { 0x1B, '[', '1', '9', '~' }, TEXT("F8"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F9)),  { 0x1B, '[', '2', '0', '~' }, TEXT("F9"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F10)), { 0x1B, '[', '2', '1', '~' }, TEXT("F10"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F11)), { 0x1B, '[', '2', '3', '~' }, TEXT("F11"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F12)), { 0x1B, '[', '2', '4', '~' }, TEXT("F12"));
		});

		It("should encode modifier parameters for both F-key branches", [this]()
		{
			// Ctrl+F1 (F1..F4 branch): ESC [ 1 ; 5 P
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F1, false, true)), { 0x1B, '[', '1', ';', '5', 'P' },      TEXT("Ctrl+F1"));
			// Ctrl+F5 (F5..F12 branch): ESC [ 15 ; 5 ~
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F5, false, true)), { 0x1B, '[', '1', '5', ';', '5', '~' }, TEXT("Ctrl+F5"));
		});
	});

	Describe("Alt meta", [this]()
	{
		It("should prefix ESC and emit the lowercase letter for Alt+letter", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F, false, false, true)), { 0x1B, 'f' }, TEXT("Alt+F"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::B, false, false, true)), { 0x1B, 'b' }, TEXT("Alt+B"));
		});

		It("should prefix ESC on top of the Ctrl-folded byte for Ctrl+Alt+letter", [this]()
		{
			// Ctrl+Alt+F -> ESC 0x06.
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F, false, true, true)), { 0x1B, 0x06 }, TEXT("Ctrl+Alt+F"));
		});

		It("should treat Windows AltGr (LeftCtrl + RightAlt) as neither Ctrl nor Alt", [this]()
		{
			// AltGr+F must fall through to OnKeyChar; translator returns empty.
			ExpectBytes(Translate(MakeAltGrKeyEvent(EKeys::F)), {}, TEXT("AltGr+F"));
		});
	});

	Describe("Modifier-only and unmapped", [this]()
	{
		It("should return empty for pure modifier presses", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::LeftShift, true)),       {}, TEXT("LeftShift alone"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::LeftControl, false, true)), {}, TEXT("LeftControl alone"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::LeftAlt, false, false, true)), {}, TEXT("LeftAlt alone"));
		});

		It("should return empty for unmodified printable keys (they belong to OnKeyChar)", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F)), {}, TEXT("Unmodified F"));
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::F, true)), {}, TEXT("Shift+F"));
		});

		It("should return empty for gamepad or other unmapped keys", [this]()
		{
			ExpectBytes(Translate(MakeTranslatorKeyEvent(EKeys::Gamepad_FaceButton_Bottom)), {}, TEXT("Gamepad face button"));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
