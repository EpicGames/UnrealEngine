// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "TerminalBuffer.h"
#include "VTParser.h"

#if WITH_DEV_AUTOMATION_TESTS

// -- Helpers --

/** Feed a raw byte sequence into the parser. */
static void ParseBytes(FVTParser& Parser, TArrayView<const uint8> Bytes)
{
	Parser.Parse(Bytes.GetData(), Bytes.Num());
}

/** Feed a string as UTF-8 bytes into the parser. */
static void ParseString(FVTParser& Parser, const FString& Text)
{
	const FTCHARToUTF8 Utf8(*Text);
	Parser.Parse(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
}

/** Get the character at a viewport-relative position. */
static TCHAR GetCharAt(FTerminalBuffer& Buffer, int32 ViewportRow, int32 Column)
{
	const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, 0);
	return Buffer.GetCell(AbsoluteRow, Column).Character;
}

/** Get the cell at a viewport-relative position. */
static const FTerminalCell& GetCellAt(FTerminalBuffer& Buffer, int32 ViewportRow, int32 Column)
{
	const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, 0);
	return Buffer.GetCell(AbsoluteRow, Column);
}

/** Get the text content of a viewport row. */
static FString GetRowText(FTerminalBuffer& Buffer, int32 ViewportRow)
{
	const int32 AbsoluteRow = Buffer.GetAbsoluteRow(ViewportRow, 0);
	return Buffer.GetTextInRange(AbsoluteRow, 0, AbsoluteRow, Buffer.GetColumns() - 1);
}

// -- Spec --

BEGIN_DEFINE_SPEC(FVTParserSpec, "Terminal.VTParser",
	EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	TUniquePtr<FTerminalBuffer> Buffer;
	TUniquePtr<FVTParser> Parser;

END_DEFINE_SPEC(FVTParserSpec)

void FVTParserSpec::Define()
{
	BeforeEach([this]()
	{
		Buffer = MakeUnique<FTerminalBuffer>();
		Buffer->Initialize(80, 24, 0);
		Parser = MakeUnique<FVTParser>();
		Parser->SetBuffer(Buffer.Get());
	});

	AfterEach([this]()
	{
		Parser.Reset();
		Buffer.Reset();
	});

	// -- A. Plain Text & C0 Controls --

	Describe("Plain Text & C0 Controls", [this]()
	{
		It("should print ASCII text at cursor and advance cursor", [this]()
		{
			ParseString(*Parser, TEXT("Hello"));
			TestEqual(TEXT("char at (0,0)"), GetCharAt(*Buffer, 0, 0), TEXT('H'));
			TestEqual(TEXT("char at (0,4)"), GetCharAt(*Buffer, 0, 4), TEXT('o'));
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 5);
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 0);
		});

		It("should handle CR by moving cursor to column 0", [this]()
		{
			ParseString(*Parser, TEXT("Hello"));
			ParseBytes(*Parser, { 0x0D }); // CR
			TestEqual(TEXT("cursor column after CR"), Buffer->Cursor.Column, 0);
			TestEqual(TEXT("cursor row after CR"), Buffer->Cursor.Row, 0);
		});

		It("should handle LF by moving cursor down one row", [this]()
		{
			ParseString(*Parser, TEXT("A"));
			ParseBytes(*Parser, { 0x0A }); // LF
			TestEqual(TEXT("cursor row after LF"), Buffer->Cursor.Row, 1);
		});

		It("should handle VT and FF as LF", [this]()
		{
			ParseBytes(*Parser, { 0x0B }); // VT
			TestEqual(TEXT("cursor row after VT"), Buffer->Cursor.Row, 1);
			ParseBytes(*Parser, { 0x0C }); // FF
			TestEqual(TEXT("cursor row after FF"), Buffer->Cursor.Row, 2);
		});

		It("should scroll when LF at bottom of viewport", [this]()
		{
			// Move cursor to last row.
			for (int32 Index = 0; Index < 23; ++Index)
			{
				ParseBytes(*Parser, { 0x0A });
			}
			TestEqual(TEXT("cursor at bottom row"), Buffer->Cursor.Row, 23);

			ParseString(*Parser, TEXT("bottom"));
			ParseBytes(*Parser, { 0x0A }); // LF at bottom should scroll

			// Cursor should still be at row 23.
			TestEqual(TEXT("cursor row after scroll"), Buffer->Cursor.Row, 23);
		});

		It("should handle BS by moving cursor left, stopping at column 0", [this]()
		{
			ParseString(*Parser, TEXT("AB"));
			ParseBytes(*Parser, { 0x08 }); // BS
			TestEqual(TEXT("cursor after BS"), Buffer->Cursor.Column, 1);

			// BS at column 0 should stop.
			Buffer->Cursor.Column = 0;
			ParseBytes(*Parser, { 0x08 });
			TestEqual(TEXT("cursor stays at 0"), Buffer->Cursor.Column, 0);
		});

		It("should handle HT by advancing to next tab stop", [this]()
		{
			ParseBytes(*Parser, { 0x09 }); // HT from column 0
			TestEqual(TEXT("tab to column 8"), Buffer->Cursor.Column, 8);

			ParseBytes(*Parser, { 0x09 }); // HT from column 8
			TestEqual(TEXT("tab to column 16"), Buffer->Cursor.Column, 16);
		});

		It("should handle BEL without crashing", [this]()
		{
			ParseBytes(*Parser, { 0x07 });
			TestEqual(TEXT("cursor unchanged"), Buffer->Cursor.Column, 0);
		});
	});

	// -- B. CSI Cursor Movement --

	Describe("CSI Cursor Movement", [this]()
	{
		It("CUU: should move cursor up", [this]()
		{
			Buffer->Cursor.Row = 5;
			ParseString(*Parser, TEXT("\x1B[3A")); // CUU 3
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 2);
		});

		It("CUU: should stop at top row", [this]()
		{
			Buffer->Cursor.Row = 2;
			ParseString(*Parser, TEXT("\x1B[10A"));
			TestEqual(TEXT("cursor row clamped at top"), Buffer->Cursor.Row, 0);
		});

		It("CUU: default parameter should be 1", [this]()
		{
			Buffer->Cursor.Row = 3;
			ParseString(*Parser, TEXT("\x1B[A"));
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 2);
		});

		It("CUD: should move cursor down", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[5B")); // CUD 5
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 5);
		});

		It("CUD: should stop at bottom row", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[100B"));
			TestEqual(TEXT("cursor row clamped at bottom"), Buffer->Cursor.Row, 23);
		});

		It("CUF: should move cursor right", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[10C")); // CUF 10
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 10);
		});

		It("CUF: should stop at right edge", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[200C"));
			TestEqual(TEXT("cursor column clamped at right"), Buffer->Cursor.Column, 79);
		});

		It("CUB: should move cursor left", [this]()
		{
			Buffer->Cursor.Column = 10;
			ParseString(*Parser, TEXT("\x1B[3D")); // CUB 3
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 7);
		});

		It("CUB: should stop at left edge", [this]()
		{
			Buffer->Cursor.Column = 2;
			ParseString(*Parser, TEXT("\x1B[10D"));
			TestEqual(TEXT("cursor column clamped at left"), Buffer->Cursor.Column, 0);
		});

		It("CHA: should set cursor to absolute column (1-based)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[20G")); // CHA column 20
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 19);
		});

		It("CHA: default should be column 1", [this]()
		{
			Buffer->Cursor.Column = 10;
			ParseString(*Parser, TEXT("\x1B[G"));
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 0);
		});

		It("CUP: should set cursor to row;col (1-based)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[10;20H")); // CUP row 10, col 20
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 9);
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 19);
		});

		It("CUP: default should be 1;1", [this]()
		{
			Buffer->Cursor.Row = 5;
			Buffer->Cursor.Column = 10;
			ParseString(*Parser, TEXT("\x1B[H"));
			TestEqual(TEXT("cursor row"), Buffer->Cursor.Row, 0);
			TestEqual(TEXT("cursor column"), Buffer->Cursor.Column, 0);
		});
	});

	// -- C. CSI Erase --

	Describe("CSI Erase", [this]()
	{
		It("ED 0: should erase from cursor to end of display", [this]()
		{
			// Fill row 0 with text.
			ParseString(*Parser, TEXT("ABCDEFGH"));
			// Move cursor to (0, 4).
			ParseString(*Parser, TEXT("\x1B[1;5H"));
			// Erase from cursor to end of display.
			ParseString(*Parser, TEXT("\x1B[0J"));

			TestEqual(TEXT("preserved before cursor"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
			TestEqual(TEXT("preserved before cursor"), GetCharAt(*Buffer, 0, 3), TEXT('D'));
			TestEqual(TEXT("erased at cursor"), GetCharAt(*Buffer, 0, 4), TEXT(' '));
			TestEqual(TEXT("erased after cursor"), GetCharAt(*Buffer, 0, 7), TEXT(' '));
		});

		It("ED 1: should erase from start of display to cursor", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEFGH"));
			ParseString(*Parser, TEXT("\x1B[1;5H")); // cursor at (0, 4)
			ParseString(*Parser, TEXT("\x1B[1J"));

			TestEqual(TEXT("erased before cursor"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
			TestEqual(TEXT("erased at cursor"), GetCharAt(*Buffer, 0, 4), TEXT(' '));
			TestEqual(TEXT("preserved after cursor"), GetCharAt(*Buffer, 0, 5), TEXT('F'));
		});

		It("ED 2: should erase entire display", [this]()
		{
			ParseString(*Parser, TEXT("Hello"));
			ParseString(*Parser, TEXT("\x1B[2J"));
			TestEqual(TEXT("cell erased"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
		});

		It("EL 0: should erase from cursor to end of line", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEFGH"));
			ParseString(*Parser, TEXT("\x1B[1;5H")); // cursor at (0, 4)
			ParseString(*Parser, TEXT("\x1B[0K"));

			TestEqual(TEXT("preserved"), GetCharAt(*Buffer, 0, 3), TEXT('D'));
			TestEqual(TEXT("erased at cursor"), GetCharAt(*Buffer, 0, 4), TEXT(' '));
			TestEqual(TEXT("erased after"), GetCharAt(*Buffer, 0, 7), TEXT(' '));
		});

		It("EL 1: should erase from start of line to cursor", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEFGH"));
			ParseString(*Parser, TEXT("\x1B[1;5H")); // cursor at (0, 4)
			ParseString(*Parser, TEXT("\x1B[1K"));

			TestEqual(TEXT("erased before"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
			TestEqual(TEXT("erased at cursor"), GetCharAt(*Buffer, 0, 4), TEXT(' '));
			TestEqual(TEXT("preserved after"), GetCharAt(*Buffer, 0, 5), TEXT('F'));
		});

		It("EL 2: should erase entire line", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEFGH"));
			ParseString(*Parser, TEXT("\x1B[1;5H"));
			ParseString(*Parser, TEXT("\x1B[2K"));

			TestEqual(TEXT("erased start"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
			TestEqual(TEXT("erased end"), GetCharAt(*Buffer, 0, 7), TEXT(' '));
		});
	});

	// -- D. CSI Insert/Delete --

	Describe("CSI Insert/Delete", [this]()
	{
		It("ICH: should insert blank characters at cursor", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEF"));
			ParseString(*Parser, TEXT("\x1B[1;3H")); // cursor at (0, 2)
			ParseString(*Parser, TEXT("\x1B[2@"));    // ICH 2

			TestEqual(TEXT("A preserved"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
			TestEqual(TEXT("B preserved"), GetCharAt(*Buffer, 0, 1), TEXT('B'));
			TestEqual(TEXT("blank inserted"), GetCharAt(*Buffer, 0, 2), TEXT(' '));
			TestEqual(TEXT("blank inserted"), GetCharAt(*Buffer, 0, 3), TEXT(' '));
			TestEqual(TEXT("C shifted right"), GetCharAt(*Buffer, 0, 4), TEXT('C'));
		});

		It("DCH: should delete characters at cursor and shift left", [this]()
		{
			ParseString(*Parser, TEXT("ABCDEF"));
			ParseString(*Parser, TEXT("\x1B[1;3H")); // cursor at (0, 2)
			ParseString(*Parser, TEXT("\x1B[2P"));    // DCH 2

			TestEqual(TEXT("A preserved"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
			TestEqual(TEXT("B preserved"), GetCharAt(*Buffer, 0, 1), TEXT('B'));
			TestEqual(TEXT("E shifted left"), GetCharAt(*Buffer, 0, 2), TEXT('E'));
			TestEqual(TEXT("F shifted left"), GetCharAt(*Buffer, 0, 3), TEXT('F'));
		});

		It("IL: should insert blank lines at cursor row", [this]()
		{
			ParseString(*Parser, TEXT("Line0"));
			ParseString(*Parser, TEXT("\x1B[2;1H")); // cursor at row 1
			ParseString(*Parser, TEXT("Line1"));
			ParseString(*Parser, TEXT("\x1B[2;1H")); // back to row 1
			ParseString(*Parser, TEXT("\x1B[1L"));    // IL 1

			// Row 1 should now be blank; original Line1 should be pushed down.
			TestEqual(TEXT("blank inserted line"), GetCharAt(*Buffer, 1, 0), TEXT(' '));
			TestEqual(TEXT("Line1 pushed down"), GetCharAt(*Buffer, 2, 0), TEXT('L'));
		});

		It("DL: should delete lines at cursor row", [this]()
		{
			ParseString(*Parser, TEXT("Line0"));
			ParseString(*Parser, TEXT("\x1B[2;1H")); // row 1
			ParseString(*Parser, TEXT("Line1"));
			ParseString(*Parser, TEXT("\x1B[3;1H")); // row 2
			ParseString(*Parser, TEXT("Line2"));
			ParseString(*Parser, TEXT("\x1B[2;1H")); // back to row 1
			ParseString(*Parser, TEXT("\x1B[1M"));    // DL 1

			// Row 1 should now have Line2 content.
			TestEqual(TEXT("Line2 moved up"), GetCharAt(*Buffer, 1, 0), TEXT('L'));
			TestEqual(TEXT("Line2 content"), GetCharAt(*Buffer, 1, 4), TEXT('2'));
		});
	});

	// -- E. Scroll --

	Describe("Scroll", [this]()
	{
		It("SU: should scroll up by Ps lines", [this]()
		{
			ParseString(*Parser, TEXT("Row0"));
			ParseString(*Parser, TEXT("\x1B[2;1H"));
			ParseString(*Parser, TEXT("Row1"));
			ParseString(*Parser, TEXT("\x1B[3;1H"));
			ParseString(*Parser, TEXT("Row2"));

			ParseString(*Parser, TEXT("\x1B[1S")); // SU 1

			// Row0 should have scrolled off; Row1 should now be at viewport row 0.
			TestEqual(TEXT("Row1 at top"), GetCharAt(*Buffer, 0, 3), TEXT('1'));
		});

		It("SD: should scroll down by Ps lines", [this]()
		{
			ParseString(*Parser, TEXT("Row0"));
			ParseString(*Parser, TEXT("\x1B[2;1H"));
			ParseString(*Parser, TEXT("Row1"));

			ParseString(*Parser, TEXT("\x1B[1T")); // SD 1

			// Row0 should now be at viewport row 1; viewport row 0 should be blank.
			TestEqual(TEXT("blank at top"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
			TestEqual(TEXT("Row0 pushed down"), GetCharAt(*Buffer, 1, 3), TEXT('0'));
		});
	});

	// -- F. SGR  - Select Graphic Rendition --

	Describe("SGR", [this]()
	{
		It("SGR 0: should reset all attributes", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[1m"));  // Bold
			ParseString(*Parser, TEXT("\x1B[31m")); // Red foreground
			ParseString(*Parser, TEXT("\x1B[0m"));  // Reset

			TestEqual(TEXT("attributes reset"), Parser->CurrentAttributes, static_cast<uint8>(ETerminalAttribute::None));
			TestEqual(TEXT("foreground reset"), Parser->CurrentForeground, FColor(212, 212, 212));
			TestEqual(TEXT("background reset"), Parser->CurrentBackground, FColor(30, 30, 30));
		});

		It("SGR 1: should set bold", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[1m"));
			TestTrue(TEXT("bold set"), (Parser->CurrentAttributes & ETerminalAttribute::Bold) != 0);
		});

		It("SGR 2: should set dim", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[2m"));
			TestTrue(TEXT("dim set"), (Parser->CurrentAttributes & ETerminalAttribute::Dim) != 0);
		});

		It("SGR 3: should set italic", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[3m"));
			TestTrue(TEXT("italic set"), (Parser->CurrentAttributes & ETerminalAttribute::Italic) != 0);
		});

		It("SGR 4: should set underline", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[4m"));
			TestTrue(TEXT("underline set"), (Parser->CurrentAttributes & ETerminalAttribute::Underline) != 0);
		});

		It("SGR 7: should set inverse", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[7m"));
			TestTrue(TEXT("inverse set"), (Parser->CurrentAttributes & ETerminalAttribute::Inverse) != 0);
		});

		It("SGR 9: should set strikethrough", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[9m"));
			TestTrue(TEXT("strikethrough set"), (Parser->CurrentAttributes & ETerminalAttribute::Strikethrough) != 0);
		});

		It("SGR 22: should clear bold and dim", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[1;2m"));  // Bold + Dim
			ParseString(*Parser, TEXT("\x1B[22m"));   // Normal intensity
			TestTrue(TEXT("bold cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Bold) == 0);
			TestTrue(TEXT("dim cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Dim) == 0);
		});

		It("SGR 23: should clear italic", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[3m"));
			ParseString(*Parser, TEXT("\x1B[23m"));
			TestTrue(TEXT("italic cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Italic) == 0);
		});

		It("SGR 24: should clear underline", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[4m"));
			ParseString(*Parser, TEXT("\x1B[24m"));
			TestTrue(TEXT("underline cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Underline) == 0);
		});

		It("SGR 27: should clear inverse", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[7m"));
			ParseString(*Parser, TEXT("\x1B[27m"));
			TestTrue(TEXT("inverse cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Inverse) == 0);
		});

		It("SGR 29: should clear strikethrough", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[9m"));
			ParseString(*Parser, TEXT("\x1B[29m"));
			TestTrue(TEXT("strikethrough cleared"), (Parser->CurrentAttributes & ETerminalAttribute::Strikethrough) == 0);
		});

		It("SGR 31: should set red foreground", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[31m"));
			TestEqual(TEXT("foreground is red"), Parser->CurrentForeground, Parser->AnsiPalette[1]);
		});

		It("SGR 42: should set green background", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[42m"));
			TestEqual(TEXT("background is green"), Parser->CurrentBackground, Parser->AnsiPalette[2]);
		});

		It("SGR 91: should set bright red foreground", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[91m"));
			TestEqual(TEXT("foreground is bright red"), Parser->CurrentForeground, Parser->AnsiPalette[9]);
		});

		It("SGR 102: should set bright green background", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[102m"));
			TestEqual(TEXT("background is bright green"), Parser->CurrentBackground, Parser->AnsiPalette[10]);
		});

		It("SGR 39: should reset to default foreground", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[31m"));  // Red
			ParseString(*Parser, TEXT("\x1B[39m"));  // Default
			TestEqual(TEXT("foreground default"), Parser->CurrentForeground, FColor(212, 212, 212));
		});

		It("SGR 49: should reset to default background", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[42m"));  // Green bg
			ParseString(*Parser, TEXT("\x1B[49m"));  // Default
			TestEqual(TEXT("background default"), Parser->CurrentBackground, FColor(30, 30, 30));
		});

		It("SGR 38;5;N: should set 256-color foreground", [this]()
		{
			// Color 196 in 6x6x6 cube: index 196 = 16 + 180 => r=5, g=0, b=0
			// r = (196 - 16) / 36 = 5, g = ((196 - 16) % 36) / 6 = 0, b = (196 - 16) % 6 = 0
			// RGB = (255, 0, 0)
			ParseString(*Parser, TEXT("\x1B[38;5;196m"));
			TestEqual(TEXT("256-color R"), Parser->CurrentForeground.R, static_cast<uint8>(255));
			TestEqual(TEXT("256-color G"), Parser->CurrentForeground.G, static_cast<uint8>(0));
			TestEqual(TEXT("256-color B"), Parser->CurrentForeground.B, static_cast<uint8>(0));
		});

		It("SGR 38;5;N: should handle grayscale ramp", [this]()
		{
			// Grayscale index 240: 8 + (240-232)*10 = 88 => RGB(88,88,88)
			ParseString(*Parser, TEXT("\x1B[38;5;240m"));
			TestEqual(TEXT("grayscale R"), Parser->CurrentForeground.R, static_cast<uint8>(88));
			TestEqual(TEXT("grayscale G"), Parser->CurrentForeground.G, static_cast<uint8>(88));
			TestEqual(TEXT("grayscale B"), Parser->CurrentForeground.B, static_cast<uint8>(88));
		});

		It("SGR 38;2;R;G;B: should set truecolor foreground", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[38;2;100;150;200m"));
			TestEqual(TEXT("truecolor R"), Parser->CurrentForeground.R, static_cast<uint8>(100));
			TestEqual(TEXT("truecolor G"), Parser->CurrentForeground.G, static_cast<uint8>(150));
			TestEqual(TEXT("truecolor B"), Parser->CurrentForeground.B, static_cast<uint8>(200));
		});

		It("should handle multiple SGR params in one sequence", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[1;31m")); // Bold + Red
			TestTrue(TEXT("bold set"), (Parser->CurrentAttributes & ETerminalAttribute::Bold) != 0);
			TestEqual(TEXT("foreground is red"), Parser->CurrentForeground, Parser->AnsiPalette[1]);
		});
	});

	// -- G. Scroll Region  - DECSTBM --

	Describe("DECSTBM Scroll Region", [this]()
	{
		It("should set scroll region with CSI Ps;Ps r", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[5;20r")); // Set region rows 5-20 (1-based)
			TestEqual(TEXT("scroll region top"), Parser->ScrollRegionTop, 4);
			TestEqual(TEXT("scroll region bottom"), Parser->ScrollRegionBottom, 19);
		});

		It("should scroll only within region on LF at bottom", [this]()
		{
			// Set scroll region to rows 2-4 (1-based), i.e. viewport rows 1-3.
			ParseString(*Parser, TEXT("\x1B[2;4r"));

			// Write text in the region.
			ParseString(*Parser, TEXT("\x1B[2;1H")); // row 2 (viewport 1)
			ParseString(*Parser, TEXT("Line1"));
			ParseString(*Parser, TEXT("\x1B[3;1H")); // row 3 (viewport 2)
			ParseString(*Parser, TEXT("Line2"));
			ParseString(*Parser, TEXT("\x1B[4;1H")); // row 4 (viewport 3)  - bottom of region
			ParseString(*Parser, TEXT("Line3"));

			// Write something on row 1 (above region) that should not scroll.
			ParseString(*Parser, TEXT("\x1B[1;1H"));
			ParseString(*Parser, TEXT("Above"));

			// Move cursor to bottom of region and LF.
			ParseString(*Parser, TEXT("\x1B[4;1H"));
			ParseBytes(*Parser, { 0x0A }); // LF

			// "Above" on row 0 should be untouched.
			TestEqual(TEXT("row above region preserved"), GetCharAt(*Buffer, 0, 0), TEXT('A'));

			// Line1 should have scrolled off; Line2 should now be at viewport row 1.
			TestEqual(TEXT("Line2 scrolled up"), GetCharAt(*Buffer, 1, 4), TEXT('2'));
		});

		It("should restore full-screen scrolling with CSI r (no params)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[5;20r"));
			ParseString(*Parser, TEXT("\x1B[r")); // Reset
			// No-param DECSTBM defaults to 1;ViewportRows, setting the region to the full viewport.
			TestEqual(TEXT("scroll region top reset"), Parser->ScrollRegionTop, 0);
			TestEqual(TEXT("scroll region bottom reset"), Parser->ScrollRegionBottom, 23);
		});
	});

	// -- H. DEC Private Modes  - DECSET/DECRST --

	Describe("DEC Private Modes", [this]()
	{
		It("DECSET ?1 / DECRST ?1: application/normal cursor keys", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1h")); // DECSET
			TestTrue(TEXT("application cursor keys on"), Parser->bApplicationCursorKeys);
			ParseString(*Parser, TEXT("\x1B[?1l")); // DECRST
			TestFalse(TEXT("application cursor keys off"), Parser->bApplicationCursorKeys);
		});

		It("DECSET ?25 / DECRST ?25: show/hide cursor", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?25l")); // Hide
			TestFalse(TEXT("cursor hidden"), Parser->bCursorVisible);
			ParseString(*Parser, TEXT("\x1B[?25h")); // Show
			TestTrue(TEXT("cursor shown"), Parser->bCursorVisible);
		});

		It("DECSET ?2004 / DECRST ?2004: bracketed paste mode", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?2004h")); // Enable
			TestTrue(TEXT("bracketed paste on"), Parser->bBracketedPaste);
			ParseString(*Parser, TEXT("\x1B[?2004l")); // Disable
			TestFalse(TEXT("bracketed paste off"), Parser->bBracketedPaste);
		});

		It("DECSET ?1049 / DECRST ?1049: alternate screen buffer", [this]()
		{
			ParseString(*Parser, TEXT("Main"));
			ParseString(*Parser, TEXT("\x1B[?1049h")); // Switch to alt buffer
			TestTrue(TEXT("alt buffer active"), Buffer->IsAlternateBufferActive());

			ParseString(*Parser, TEXT("\x1B[?1049l")); // Restore main buffer
			TestFalse(TEXT("main buffer restored"), Buffer->IsAlternateBufferActive());
			TestEqual(TEXT("main content preserved"), GetCharAt(*Buffer, 0, 0), TEXT('M'));
		});

		It("DECSET ?2026 / DECRST ?2026: synchronized output", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?2026h"));
			TestTrue(TEXT("synchronized output on"), Parser->bSynchronizedOutput);
			ParseString(*Parser, TEXT("\x1B[?2026l"));
			TestFalse(TEXT("synchronized output off"), Parser->bSynchronizedOutput);
		});

		It("DECSET ?1000 / DECRST ?1000: normal mouse tracking", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1000h"));
			TestTrue(TEXT("normal tracking on"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::Normal);
			ParseString(*Parser, TEXT("\x1B[?1000l"));
			TestTrue(TEXT("tracking off"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::None);
		});

		It("DECSET ?1002 / DECRST ?1002: button-event mouse tracking", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1002h"));
			TestTrue(TEXT("button-event tracking on"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::ButtonEvent);
			ParseString(*Parser, TEXT("\x1B[?1002l"));
			TestTrue(TEXT("tracking off"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::None);
		});

		It("DECSET ?1003 / DECRST ?1003: any-event mouse tracking", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1003h"));
			TestTrue(TEXT("any-event tracking on"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::Any);
			ParseString(*Parser, TEXT("\x1B[?1003l"));
			TestTrue(TEXT("tracking off"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::None);
		});

		It("DECSET ?1006 / DECRST ?1006: SGR extended mouse encoding", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1006h"));
			TestTrue(TEXT("SGR encoding on"), Parser->bSGRMouseEncoding);
			ParseString(*Parser, TEXT("\x1B[?1006l"));
			TestFalse(TEXT("SGR encoding off"), Parser->bSGRMouseEncoding);
		});

		It("should set multiple modes in one sequence", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[?1000;1006h"));
			TestTrue(TEXT("normal tracking on"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::Normal);
			TestTrue(TEXT("SGR encoding on"), Parser->bSGRMouseEncoding);
			ParseString(*Parser, TEXT("\x1B[?1000;1006l"));
			TestTrue(TEXT("tracking off"), Parser->MouseTrackingMode == FVTParser::EMouseTrackingMode::None);
			TestFalse(TEXT("SGR encoding off"), Parser->bSGRMouseEncoding);
		});
	});

	// -- I. OSC  - Operating System Command --

	Describe("OSC", [this]()
	{
		It("OSC 0: should set window title (BEL terminated)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B]0;My Title\x07"));
			TestEqual(TEXT("window title"), Parser->WindowTitle, TEXT("My Title"));
		});

		It("OSC 2: should set window title (BEL terminated)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B]2;Another Title\x07"));
			TestEqual(TEXT("window title"), Parser->WindowTitle, TEXT("Another Title"));
		});

		It("OSC: should accept ST terminator (ESC backslash)", [this]()
		{
			ParseString(*Parser, TEXT("\x1B]0;ST Title\x1B\\"));
			TestEqual(TEXT("window title via ST"), Parser->WindowTitle, TEXT("ST Title"));
		});
	});

	// -- J. Device Status Report / Device Attributes --

	Describe("Device Status Report", [this]()
	{
		It("DSR 5n: should report status OK", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[5n"));
			FString Response(reinterpret_cast<const char*>(Parser->ResponseBuffer.GetData()), Parser->ResponseBuffer.Num());
			TestTrue(TEXT("response contains 0n"), Response.Contains(TEXT("0n")));
		});

		It("DSR 6n: should report cursor position (1-based)", [this]()
		{
			Buffer->Cursor.Row = 4;
			Buffer->Cursor.Column = 9;
			ParseString(*Parser, TEXT("\x1B[6n"));
			FString Response(reinterpret_cast<const char*>(Parser->ResponseBuffer.GetData()), Parser->ResponseBuffer.Num());
			// Should report row 5, column 10 (1-based).
			TestTrue(TEXT("response contains 5;10R"), Response.Contains(TEXT("5;10R")));
		});

		It("Primary DA: should return a response", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[c"));
			TestTrue(TEXT("response buffer not empty"), Parser->ResponseBuffer.Num() > 0);
		});
	});

	// -- K. Escape Sequences --

	Describe("Escape Sequences", [this]()
	{
		It("ESC 7 / ESC 8: should save and restore cursor position", [this]()
		{
			Buffer->Cursor.Row = 5;
			Buffer->Cursor.Column = 10;
			ParseString(*Parser, TEXT("\x1B\x37")); // ESC 7 (DECSC)

			Buffer->Cursor.Row = 0;
			Buffer->Cursor.Column = 0;
			ParseString(*Parser, TEXT("\x1B\x38")); // ESC 8 (DECRC)

			TestEqual(TEXT("restored row"), Buffer->Cursor.Row, 5);
			TestEqual(TEXT("restored column"), Buffer->Cursor.Column, 10);
		});

		It("ESC H: should set tab stop at current column", [this]()
		{
			// Clear all tab stops first.
			ParseString(*Parser, TEXT("\x1B[3g"));

			// Set a tab stop at column 5.
			Buffer->Cursor.Column = 5;
			ParseString(*Parser, TEXT("\x1BH")); // HTS

			// Move to column 0 and tab.
			Buffer->Cursor.Column = 0;
			ParseBytes(*Parser, { 0x09 }); // HT
			TestEqual(TEXT("tab to custom stop"), Buffer->Cursor.Column, 5);
		});

		It("CSI 0g: should clear tab stop at current column", [this]()
		{
			// Default tab stops at every 8 columns.
			Buffer->Cursor.Column = 8;
			ParseString(*Parser, TEXT("\x1B[0g")); // Clear tab at column 8

			Buffer->Cursor.Column = 0;
			ParseBytes(*Parser, { 0x09 }); // HT
			// Should skip column 8 and go to 16.
			TestEqual(TEXT("tab skips cleared stop"), Buffer->Cursor.Column, 16);
		});

		It("CSI 3g: should clear all tab stops", [this]()
		{
			ParseString(*Parser, TEXT("\x1B[3g"));

			Buffer->Cursor.Column = 0;
			ParseBytes(*Parser, { 0x09 }); // HT
			// No tab stops  - should go to last column.
			TestEqual(TEXT("tab goes to end"), Buffer->Cursor.Column, 79);
		});
	});

	// -- L. UTF-8 Decoding --

	Describe("UTF-8 Decoding", [this]()
	{
		It("should decode 2-byte sequence (U+00E9, e with acute)", [this]()
		{
			const uint8 Bytes[] = { 0xC3, 0xA9 };
			ParseBytes(*Parser, Bytes);
			TestEqual(TEXT("character is e-acute"), GetCharAt(*Buffer, 0, 0), static_cast<TCHAR>(0x00E9));
		});

		It("should decode 3-byte sequence (U+2713, check mark)", [this]()
		{
			const uint8 Bytes[] = { 0xE2, 0x9C, 0x93 };
			ParseBytes(*Parser, Bytes);
			TestEqual(TEXT("character is check mark"), GetCharAt(*Buffer, 0, 0), static_cast<TCHAR>(0x2713));
		});

		It("should decode 4-byte sequence (U+1F600, emoji)", [this]()
		{
			const uint8 Bytes[] = { 0xF0, 0x9F, 0x98, 0x80 };
			ParseBytes(*Parser, Bytes);
			// 4-byte codepoints on platforms with 16-bit TCHAR get stored as surrogate pairs
			// or implementation-defined. Just verify no crash and cursor advanced.
			TestTrue(TEXT("cursor advanced"), Buffer->Cursor.Column >= 1);
		});

		It("should recover from invalid continuation byte", [this]()
		{
			// Start a 2-byte sequence, then send an ASCII char instead of continuation.
			const uint8 Bytes[] = { 0xC3, 0x41 }; // 0xC3 expects continuation, 0x41 = 'A'
			ParseBytes(*Parser, Bytes);
			// Should recover and print 'A'.
			TestEqual(TEXT("recovered with A"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
		});

		It("should reject overlong encoding (C0 80 = overlong NUL)", [this]()
		{
			const uint8 Bytes[] = { 0xC0, 0x80, 0x41 }; // Overlong NUL, then 'A'
			ParseBytes(*Parser, Bytes);
			// Overlong should be rejected. 'A' should be at column 0.
			TestEqual(TEXT("overlong rejected, A printed"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
		});
	});

	// -- M. Parser State Machine Resilience --

	Describe("Parser State Machine Resilience", [this]()
	{
		It("CAN mid-sequence should return to ground state", [this]()
		{
			// Start a CSI sequence, then CAN, then print.
			const uint8 Bytes[] = { 0x1B, 0x5B, 0x31, 0x18, 0x41 }; // ESC [ 1 CAN A
			ParseBytes(*Parser, Bytes);
			// 'A' should be printed normally.
			TestEqual(TEXT("A printed after CAN"), GetCharAt(*Buffer, 0, 0), TEXT('A'));
		});

		It("SUB mid-sequence should return to ground state", [this]()
		{
			const uint8 Bytes[] = { 0x1B, 0x5B, 0x31, 0x1A, 0x42 }; // ESC [ 1 SUB B
			ParseBytes(*Parser, Bytes);
			TestEqual(TEXT("B printed after SUB"), GetCharAt(*Buffer, 0, 0), TEXT('B'));
		});

		It("ESC mid-sequence should start new escape sequence", [this]()
		{
			// ESC [ 1 ESC [ 2 J  - the first CSI is abandoned, second is ED 2.
			ParseString(*Parser, TEXT("Hello"));
			ParseString(*Parser, TEXT("\x1B[1\x1B[2J"));

			// ED 2 should have cleared the display.
			TestEqual(TEXT("display cleared"), GetCharAt(*Buffer, 0, 0), TEXT(' '));
		});

		It("malformed CSI with private marker mid-params should be ignored", [this]()
		{
			// CSI 1 ? 2 m  - a '?' appearing after the first parameter position
			// triggers CSI ignore per the Williams spec (0x3C-0x3F after first pos).
			ParseString(*Parser, TEXT("\x1B[1?2m"));
			ParseString(*Parser, TEXT("X"));
			// After the ignored sequence, 'X' should print normally.
			TestEqual(TEXT("X printed after ignored CSI"), GetCharAt(*Buffer, 0, 0), TEXT('X'));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
