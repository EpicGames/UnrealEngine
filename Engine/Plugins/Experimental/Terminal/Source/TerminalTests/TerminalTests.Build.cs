// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TerminalTests : ModuleRules
{
	public TerminalTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"Terminal",
				"UnrealEd"
			});
	}
}
