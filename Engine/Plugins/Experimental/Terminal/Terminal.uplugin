{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Terminal",
	"Description": "Native Slate terminal emulator.",
	"NoRedist": true,
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "Terminal",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TerminalTests",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
